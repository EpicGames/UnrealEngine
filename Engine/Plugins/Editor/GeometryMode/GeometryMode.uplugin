{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "GeometryMode",
	"Description": "Geometry and BSP editing",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "GeometryMode",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "BspMode",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "TextureAlignMode",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}