// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BspMode : ModuleRules
{
	public BspMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
				"PropertyEditor",
				"PlacementMode",
			}
        );

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
				"PlacementMode",
			}
        );
	}
}
