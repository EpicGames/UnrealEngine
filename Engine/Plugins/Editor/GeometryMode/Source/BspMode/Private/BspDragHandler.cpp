// Copyright Epic Games, Inc. All Rights Reserved.

#include "BspDragHandler.h"
#include "EditorClassUtils.h"
#include "BspModeModule.h"
#include "DragAndDrop/BrushBuilderDragDropOp.h"

FBspDragHandler::FBspDragHandler()
{
}

void FBspDragHandler::Initialize( TSharedRef<FBspBuilderType> BspBuilder )
{
	ToolTip = FEditorClassUtils::GetTooltip(ABrush::StaticClass(), BspBuilder->ToolTipText );
	IconBrush = BspBuilder->Icon; 
	
	GetContentToDrag.BindLambda( [ BspBuilder ] ()
	{
		const bool bIsAdditive = true;
		
		const TWeakObjectPtr<UBrushBuilder> ActiveBrushBuilder = GEditor->FindBrushBuilder(BspBuilder->BuilderClass.Get());
		return FBrushBuilderDragDropOp::New( ActiveBrushBuilder, BspBuilder->Icon, bIsAdditive );
	});
}