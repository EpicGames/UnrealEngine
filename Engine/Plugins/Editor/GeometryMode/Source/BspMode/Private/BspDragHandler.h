// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPlacementModeModule.h"

struct FBspBuilderType;

class FBspDragHandler : public FPlaceableItem::FDragHandler
{
public:

	FBspDragHandler();
	void Initialize( TSharedRef<FBspBuilderType> InBspBuilder );
};
