// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdMode.h"

/**
 * Mode Toolkit for the Bsp
 */
class FBspMode : public FEdMode
{
public:

	virtual bool UsesToolkits() const override { return false; }

	virtual bool UsesPropertyWidgets() const override { return true; }
};
