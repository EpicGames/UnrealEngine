// Copyright Epic Games, Inc. All Rights Reserved.

#include "BspModeModule.h"
#include "Styling/AppStyle.h"
#include "Builders/ConeBuilder.h"
#include "Builders/CubeBuilder.h"
#include "Builders/CurvedStairBuilder.h"
#include "Builders/CylinderBuilder.h"
#include "Builders/LinearStairBuilder.h"
#include "Builders/SpiralStairBuilder.h"
#include "Builders/TetrahedronBuilder.h"
#include "BspModeStyle.h"
#include "IPlacementModeModule.h"
#include "DragAndDrop/BrushBuilderDragDropOp.h"
#include "BspDragHandler.h"
#include "IPlacementModeModule.h"

#define LOCTEXT_NAMESPACE "BspMode"

FBspBuilderType::FBspBuilderType(
	UClass* InBuilderClass,
	const FText& InText,
	const FText& InToolTipText,
	const FSlateBrush* InIcon )
:	BuilderClass(InBuilderClass)
,	BuilderClassName(InBuilderClass ? InBuilderClass->GetFName() : NAME_None)
,	Text(InText)
,	ToolTipText(InToolTipText)
,	Icon(InIcon)
{
}

void FBspModeModule::StartupModule()
{
	FBspModeStyle::Initialize();
	CategoryName = "Geometry";
/*
	FEditorModeRegistry::Get().RegisterMode<FBspMode>(
		FGeometryEditingModes::EM_Bsp,
		NSLOCTEXT("GeometryMode", "DisplayName", "Geometry Editing"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.BspMode", "LevelEditor.BspMode.Small"),
		false,		// Visible
		100			// UI priority order
		);
*/

	FPlacementCategoryInfo CategoryInfo (LOCTEXT("PlacementMode_Geometry", "Geometry"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.BSP"), CategoryName, TEXT("PMGeometry"), 35);
	IPlacementModeModule::Get().RegisterPlacementCategory( CategoryInfo );
	
	RegisterBspBuilderType(UCubeBuilder::StaticClass(), LOCTEXT("CubeBuilderName", "Box"), LOCTEXT("CubeBuilderToolTip", "Make a box brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.BoxBrush")));
	RegisterBspBuilderType(UConeBuilder::StaticClass(), LOCTEXT("ConeBuilderName", "Cone"), LOCTEXT("ConeBuilderToolTip", "Make a cone brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.ConeBrush")));
	RegisterBspBuilderType(UCylinderBuilder::StaticClass(), LOCTEXT("CylinderBuilderName", "Cylinder"), LOCTEXT("CylinderBuilderToolTip", "Make a cylinder brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.CylinderBrush")));
	RegisterBspBuilderType(UCurvedStairBuilder::StaticClass(), LOCTEXT("CurvedStairBuilderName", "Curved Stair"), LOCTEXT("CurvedStairBuilderToolTip", "Make a curved stair brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.CurvedStairBrush")));
	RegisterBspBuilderType(ULinearStairBuilder::StaticClass(), LOCTEXT("LinearStairBuilderName", "Linear Stair"), LOCTEXT("LinearStairBuilderToolTip", "Make a linear stair brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.LinearStairBrush")));
	RegisterBspBuilderType(USpiralStairBuilder::StaticClass(), LOCTEXT("SpiralStairBuilderName", "Spiral Stair"), LOCTEXT("SpiralStairBuilderToolTip", "Make a spiral stair brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.SpiralStairBrush")));
	RegisterBspBuilderType(UTetrahedronBuilder::StaticClass(), LOCTEXT("SphereBuilderName", "Sphere"), LOCTEXT("SphereBuilderToolTip", "Make a sphere brush"), FBspModeStyle::Get().GetBrush(TEXT("BspMode.SphereBrush")));
}

void FBspModeModule::ShutdownModule()
{
	IPlacementModeModule::Get().UnregisterPlacementCategory( CategoryName );
	for (const FName BuilderClassName : BspBuilderTypesToUnregisterOnShutdown)
	{
		UnregisterBspBuilderType(BuilderClassName);
	}
	BspBuilderTypes.Empty();
	BspBuilderTypesToUnregisterOnShutdown.Empty();
}


void FBspModeModule::RegisterBspBuilderType( class UClass* InBuilderClass, const FText& InBuilderName, const FText& InBuilderTooltip, const FSlateBrush* InBuilderIcon )
{
	check(InBuilderClass->IsChildOf(UBrushBuilder::StaticClass()));
	const TSharedPtr<FBspBuilderType> BuilderType = MakeShareable(new FBspBuilderType(InBuilderClass, InBuilderName, InBuilderTooltip, InBuilderIcon));
	BspBuilderTypes.Add( BuilderType );
	const TSharedPtr<FBspDragHandler> Handler = MakeShared<FBspDragHandler>();
	Handler->Initialize( BuilderType.ToSharedRef() );
	const FString Name = "BSP_" + BuilderType->Text.ToString();

	static int32 SortOrder = 0;
	
	BuilderType->PlaceableItem =  MakeShared<FPlaceableItem>( Handler, SortOrder++, BuilderType->Text, Name );
	BuilderType->PlacementModeID = IPlacementModeModule::Get().RegisterPlaceableItem( CategoryName, BuilderType->PlaceableItem.ToSharedRef() );

	BspBuilderTypesToUnregisterOnShutdown.Add( BuilderType->BuilderClassName );
}


void FBspModeModule::UnregisterBspBuilderType( const FName InBuilderClassName )
{
	for ( TSharedPtr<FBspBuilderType> BuilderType : BspBuilderTypes )
	{
		if ( BuilderType.IsValid() && BuilderType->PlaceableItem.IsValid() && BuilderType->BuilderClassName == InBuilderClassName )
		{
			if ( BuilderType->PlacementModeID.IsSet() )
			{
				IPlacementModeModule::Get().UnregisterPlaceableItem( BuilderType->PlacementModeID.GetValue() );
			}
		}
	}
	// If UObject system is not initialized the below code cannot run because it creates weak object pointers which crashes
	// the internal UObject array.
	if (UObjectInitialized())
	{
		BspBuilderTypes.RemoveAll(
			[InBuilderClassName](const TSharedPtr<FBspBuilderType>& RemovalCandidate) -> bool
			{
				return RemovalCandidate.IsValid() && RemovalCandidate->BuilderClassName == InBuilderClassName;
			}
		);
	}
}


const TArray< TSharedPtr<FBspBuilderType> >& FBspModeModule::GetBspBuilderTypes()
{
	return BspBuilderTypes;
}


TSharedPtr<FBspBuilderType> FBspModeModule::FindBspBuilderType(UClass* InBuilderClass) const
{
	const TSharedPtr<FBspBuilderType>* FoundBuilder = BspBuilderTypes.FindByPredicate(
		[InBuilderClass] ( const TSharedPtr<FBspBuilderType>& FindCandidate ) -> bool
		{
			return (FindCandidate->BuilderClass == InBuilderClass);
		}
	);

	return FoundBuilder != nullptr ? *FoundBuilder : TSharedPtr<FBspBuilderType>();
}

IMPLEMENT_MODULE( FBspModeModule, BspMode );

#undef LOCTEXT_NAMESPACE
