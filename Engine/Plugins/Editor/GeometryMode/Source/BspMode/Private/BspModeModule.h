// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "IBspModeModule.h"
#include "IPlacementModeModule.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"

struct FSlateBrush;
struct FPlaceableItem;
struct FDragHandler;
class FBrushBuilderDragDropOp;

struct FBspBuilderType
{
	FBspBuilderType(class UClass* InBuilderClass, const FText& InText, const FText& InToolTipText, const FSlateBrush* InIcon);

	/** The class of the builder brush */
	TWeakObjectPtr<UClass> BuilderClass;

	/** The class name of the builder brush */
	FName BuilderClassName;

	/** The name to be displayed for this builder */
	FText Text;

	/** The name to be displayed for this builder */
	FText ToolTipText;

	/** The icon to be displayed for this builder */
	const FSlateBrush* Icon;

	/** the placeable item that will provide the information for the draggable  for this BSP builder */
	TSharedPtr<FPlaceableItem> PlaceableItem;

	/** TOptional<FPlacementModeID>for deletion */
	TOptional<FPlacementModeID> PlacementModeID;
};

class FBspModeModule : public IBspModeModule
{
public:
	
	/** IModuleInterface interface */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** IBspModeModule interface */
	virtual void RegisterBspBuilderType( class UClass* InBuilderClass, const FText& InBuilderName, const FText& InBuilderTooltip, const FSlateBrush* InBuilderIcon ) override;
	virtual void UnregisterBspBuilderType( class FName InBuilderClassName ) override;

	const TArray< TSharedPtr<FBspBuilderType> >& GetBspBuilderTypes();
	TSharedPtr<FBspBuilderType> FindBspBuilderType(UClass* InBuilderClass) const;

private:

	TArray< TSharedPtr<FBspBuilderType> > BspBuilderTypes;
	FName CategoryName;
	
	TArray<FName> BspBuilderTypesToUnregisterOnShutdown;
};
