// Copyright Epic Games, Inc. All Rights Reserved.

#include "BspModeStyle.h"
#include "Brushes/SlateBorderBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateTypes.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Brushes/SlateNoResource.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"


TSharedPtr< FSlateStyleSet > FBspModeStyle::StyleSet = nullptr;

#define RootToContentDir StyleSet->RootToContentDir

void FBspModeStyle::Initialize()
{
	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable( new FSlateStyleSet("BspModeStyle") );
	StyleSet->SetContentRoot( FPaths::EngineContentDir() / TEXT("Editor/Slate") );
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Note, these sizes are in Slate Units.
	// Slate Units do NOT have to map to pixels.
	const FVector2D Icon5x16(5.0f, 16.0f);
	const FVector2D Icon8x4(8.0f, 4.0f);
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon12x16(12.0f, 16.0f);
	const FVector2D Icon14x14(14.0f, 14.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon22x22(22.0f, 22.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon25x25(25.0f, 25.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);
	const FVector2D Icon36x24(36.0f, 24.0f);
	const FVector2D Icon128x128(128.0f, 128.0f);

	// We need some colors from Editor Style & this is the only way to do this at the moment
	const FSlateColor DefaultForeground = FAppStyle::GetSlateColor("DefaultForeground");
	const FSlateColor InvertedForeground = FAppStyle::GetSlateColor("InvertedForeground");
	const FSlateColor SelectorColor = FAppStyle::GetSlateColor("SelectorColor");
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Inactive = FAppStyle::GetSlateColor("SelectionColor_Inactive");
	
	// Normal Text
	FTextBlockStyle NormalText = FTextBlockStyle()
		.SetFont( DEFAULT_FONT("Regular", 9 ) )
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor( FLinearColor( 0.02f, 0.3f, 0.0f ) )
		.SetHighlightShape( BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f/8.f) ) );

	StyleSet->Set( "BspMode.BoxBrush", new IMAGE_BRUSH( "Icons/icon_box_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.BoxBrush.Small", new IMAGE_BRUSH( "Icons/icon_box_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.ConeBrush", new IMAGE_BRUSH( "Icons/icon_cone_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.ConeBrush.Small", new IMAGE_BRUSH( "Icons/icon_cone_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.CylinderBrush", new IMAGE_BRUSH( "Icons/icon_cylinder_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.CylinderBrush.Small", new IMAGE_BRUSH( "Icons/icon_cylinder_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.CurvedStairBrush", new IMAGE_BRUSH( "Icons/icon_curvedstair_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.CurvedStairBrush.Small", new IMAGE_BRUSH( "Icons/icon_curvedstair_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.LinearStairBrush", new IMAGE_BRUSH( "Icons/icon_stair_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.LinearStairBrush.Small", new IMAGE_BRUSH( "Icons/icon_stair_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.SpiralStairBrush", new IMAGE_BRUSH( "Icons/icon_spiralstair_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.SpiralStairBrush.Small", new IMAGE_BRUSH( "Icons/icon_spiralstair_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.VolumetricBrush", new IMAGE_BRUSH( "Icons/icon_volumetric_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.VolumetricBrush.Small", new IMAGE_BRUSH( "Icons/icon_volumetric_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.SphereBrush", new IMAGE_BRUSH( "Icons/icon_sphere_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.SphereBrush.Small", new IMAGE_BRUSH( "Icons/icon_sphere_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.SheetBrush", new IMAGE_BRUSH( "Icons/icon_plane_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.SheetBrush.Small", new IMAGE_BRUSH( "Icons/icon_plane_40x", Icon20x20 ) );

	StyleSet->Set( "BspMode.CSGAdd", new IMAGE_BRUSH( "Icons/icon_csgadd_40x", Icon40x40) );
	StyleSet->Set( "BspMode.CSGAdd.Small", new IMAGE_BRUSH( "Icons/icon_csgadd_40x", Icon20x20) );
	StyleSet->Set( "BspMode.CSGSubtract", new IMAGE_BRUSH( "Icons/icon_csgsubtract_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.CSGSubtract.Small", new IMAGE_BRUSH( "Icons/icon_csgsubtract_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.CSGIntersect", new IMAGE_BRUSH( "Icons/icon_csgintersect_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.CSGIntersect.Small", new IMAGE_BRUSH( "Icons/icon_csgintersect_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.CSGDeintersect", new IMAGE_BRUSH( "Icons/icon_csgdeIntersect_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.CSGDeintersect.Small", new IMAGE_BRUSH( "Icons/icon_csgdeIntersect_40x", Icon20x20 ) );

	StyleSet->Set( "BspMode.BrushEditMode", new IMAGE_BRUSH( "Icons/icon_editgeometry_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.BrushEditMode.Small", new IMAGE_BRUSH( "Icons/icon_editgeometry_40x", Icon20x20 ) );
	StyleSet->Set( "BspMode.TextureAlignmentMode", new IMAGE_BRUSH( "Icons/icon_texturealign_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.TextureAlignmentMode.Small", new IMAGE_BRUSH( "Icons/icon_texturealign_40x", Icon20x20 ) );

	StyleSet->Set( "BspMode.AddVolume", new IMAGE_BRUSH( "Icons/icon_volume_40x", Icon40x40 ) );
	StyleSet->Set( "BspMode.AddVolume.Small", new IMAGE_BRUSH( "Icons/icon_volume_40x", Icon20x20 ) );

	StyleSet->Set( "BspMode.TableRow", FTableRowStyle()
		.SetEvenRowBackgroundBrush( FSlateNoResource() )
		.SetEvenRowBackgroundHoveredBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f) ) )
		.SetOddRowBackgroundBrush( FSlateNoResource() )
		.SetOddRowBackgroundHoveredBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f) ) )
		.SetSelectorFocusedBrush( BORDER_BRUSH( "Common/Selector", FMargin(4.f/16.f), SelectorColor ) )
		.SetActiveBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, SelectionColor ) )
		.SetActiveHoveredBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, SelectionColor ) )
		.SetInactiveBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, SelectionColor_Inactive ) )
		.SetInactiveHoveredBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, SelectionColor_Inactive ) )
		.SetTextColor( DefaultForeground )
		.SetSelectedTextColor( InvertedForeground )
		);

	StyleSet->Set( "BspMode.ThumbnailShadow", new BOX_BRUSH( "ContentBrowser/ThumbnailShadow" , FMargin( 4.0f / 64.0f ) ) );
	StyleSet->Set( "BspMode.ThumbnailBackground", new IMAGE_BRUSH( "Common/ClassBackground_64x", FVector2D(64.f, 64.f), FLinearColor(0.75f, 0.75f, 0.75f, 1.0f) ) );
	StyleSet->Set( "BspMode.ThumbnailText", NormalText );

	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

#undef RootToContentDir

void FBspModeStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

const ISlateStyle& FBspModeStyle::Get()
{
	return *( StyleSet.Get() );
}

const FName& FBspModeStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}
