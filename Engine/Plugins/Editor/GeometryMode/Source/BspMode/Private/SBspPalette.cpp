// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBspPalette.h"
#include "BspModeModule.h"
#include "Modules/ModuleManager.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"
#include "DragAndDrop/BrushBuilderDragDropOp.h"
#include "EditorClassUtils.h"

#define LOCTEXT_NAMESPACE "BspPalette"

/** The list view mode of the asset view */
class SBspBuilderListView : public SListView<TSharedPtr<FBspBuilderType>>
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent ) override
	{
		return FReply::Unhandled();
	}
};

class SBspButton: public SButton
{
	SLATE_BEGIN_ARGS(SBspButton) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FBspBuilderType>& InBspBuilder)
	{
		BspBuilder = InBspBuilder;	
		SButton::Construct( SButton::FArguments()
			.ButtonStyle(&FAppStyle::Get(), "PlacementBrowser.Asset")
			.Cursor( EMouseCursor::GrabHand )
		);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			if (IsEnabled())
			{
				Press();
			}

			return FReply::Handled().DetectDrag( SharedThis( this ), MouseEvent.GetEffectingButton() );
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SButton::OnMouseButtonUp(MyGeometry, MouseEvent);
		return FReply::Unhandled();	
	}

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{

		if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
		{
			if(BspBuilder.IsValid())
			{
				TWeakObjectPtr<UBrushBuilder> ActiveBrushBuilder = GEditor->FindBrushBuilder(BspBuilder->BuilderClass.Get());
				if (ActiveBrushBuilder.IsValid())
				{
					return FReply::Handled().BeginDragDrop(FBrushBuilderDragDropOp::New(ActiveBrushBuilder, BspBuilder->Icon, true /*IsAdditive*/));
				}
			}
		}

		return FReply::Unhandled();
	}

	TSharedPtr<FBspBuilderType> BspBuilder;
};

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBspPalette::Construct( const FArguments& InArgs )
{
	FBspModeModule& BspModeModule = FModuleManager::GetModuleChecked<FBspModeModule>("BspMode");

	ListViewWidget = 
		SNew(SBspBuilderListView)
		.ListItemsSource(&BspModeModule.GetBspBuilderTypes())
		.OnGenerateRow(this, &SBspPalette::MakeListViewWidget);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, ListViewWidget.ToSharedRef())
			[
				ListViewWidget.ToSharedRef()
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<ITableRow> SBspPalette::MakeListViewWidget(TSharedPtr<FBspBuilderType> BspBuilder, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(BspBuilder.IsValid());
	check(BspBuilder->BuilderClass.IsValid());

	TSharedRef< STableRow<TSharedPtr<FBspBuilderType>> > TableRowWidget = 
		SNew( STableRow<TSharedPtr<FBspBuilderType>>, OwnerTable )
		.Style(&FAppStyle::Get(), "PlacementBrowser.PlaceableItemRow")
		.Padding(FMargin(8.f, 2.f, 12.f, 2.f));


	TSharedRef<SWidget> Content = 
		SNew(SOverlay)

		+SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::Get().GetBrush("PlacementBrowser.Asset.Background"))
			.Cursor( EMouseCursor::GrabHand )
			.Padding(0)
			[

				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.Padding(8.0f, 4.f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew( SBox )
					.WidthOverride(40)
					.HeightOverride(40)
					[

						SNew(SImage)
						.Image(BspBuilder->Icon)
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.Padding(0)
				[

					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("PlacementBrowser.Asset.LabelBack"))
					.Padding(FMargin(9, 0, 0, 1))
					.VAlign(VAlign_Center)
					[
							SNew( STextBlock )
							.TextStyle( FAppStyle::Get(), "PlacementBrowser.Asset.Name" )
							.Text(BspBuilder->Text)
					]
				]
			]
		]


		+SOverlay::Slot()
		[
			SNew(SBspButton, BspBuilder)
			.ToolTip(FEditorClassUtils::GetTooltip(ABrush::StaticClass(), BspBuilder->ToolTipText))
		];

	TableRowWidget->SetContent(Content);

	return TableRowWidget;
}

#undef LOCTEXT_NAMESPACE
