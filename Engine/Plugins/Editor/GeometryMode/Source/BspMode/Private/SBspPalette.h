// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"

class IDetailsView;
class ITableRow;
class STableViewBase;
class UBrushBuilder;
class SBspBuilderListView;
struct FSlateBrush;
enum class ECheckBoxState : uint8;

class SBspPalette : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS( SBspPalette ){}
	SLATE_END_ARGS();

	void Construct( const FArguments& InArgs );

private:

	/** Make a widget for the list view display */
	TSharedRef<ITableRow> MakeListViewWidget(TSharedPtr<struct FBspBuilderType> BspBuilder, const TSharedRef<STableViewBase>& OwnerTable);


	/** List View for the BSP Builder Types */
	TSharedPtr<class SBspBuilderListView> ListViewWidget;

};
