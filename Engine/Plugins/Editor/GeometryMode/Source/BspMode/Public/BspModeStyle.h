// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

#define UE_API BSPMODE_API

class ISlateStyle;

/**
 * BSP mode slate style
 */
class FBspModeStyle
{
public:

	static UE_API void Initialize();
	static UE_API void Shutdown();

	static UE_API const ISlateStyle& Get();

	static UE_API const FName& GetStyleSetName();

private:

	/** Singleton instances of this style. */
	static UE_API TSharedPtr< class FSlateStyleSet > StyleSet;
};

#undef UE_API
