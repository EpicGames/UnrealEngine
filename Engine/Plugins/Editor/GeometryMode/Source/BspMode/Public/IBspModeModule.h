// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class FText;

struct FSlateBrush;

class IBspModeModule : public IModuleInterface
{
public:
	virtual void RegisterBspBuilderType( class UClass* InBuilderClass, const FText& InBuilderName, const FText& InBuilderTooltip, const FSlateBrush* InBuilderIcon ) = 0;
	UE_DEPRECATED(5.8, "Use the FName variant of UnregisterBspBuilderType")
	virtual void UnregisterBspBuilderType( class UClass* InBuilderClass ) { }
	virtual void UnregisterBspBuilderType( FName InBuilderClassName ) = 0;
};
