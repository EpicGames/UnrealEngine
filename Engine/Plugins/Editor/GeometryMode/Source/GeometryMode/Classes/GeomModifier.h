// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Base class of all geometry mode modifiers.
 */

#include "UObject/Package.h"
#include "GeomModifier.generated.h"

enum EInputEvent : int;
struct FKey;

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;

typedef TSharedPtr<class FGeomObject> FGeomObjectPtr;

UCLASS(abstract, hidecategories=(Object, GeomModifier))
class UGeomModifier : public UObject
{
	GENERATED_UCLASS_BODY()

	/** A human readable name for this modifier (appears on buttons, menus, etc) */
	UPROPERTY(EditAnywhere, Category=GeomModifier)
	FText Description;

	/** The tooltip to be displayed for this modifier */
	UPROPERTY(EditAnywhere, Category = GeomModifier)
	FText Tooltip;

	/** Icon name for showing this modifier in a toolbar */
	UPROPERTY()
	FName ToolbarIconName;

	/** If true, this modifier should be displayed as a push button instead of a radio button */
	UPROPERTY(EditAnywhere, Category=GeomModifier)
	uint32 bPushButton:1;

	/**
	 * true if the modifier has been initialized.
	 * This is useful for interpreting user input and mouse drags correctly.
	 */
	UPROPERTY(EditAnywhere, Category=GeomModifier)
	uint32 bInitialized:1;

	/** If true, the pivot offset should be updated when the modification ends */
	UPROPERTY()
	uint32 bPendingPivotOffsetUpdate:1;

	UPROPERTY()
	uint32 bAppearsInToolbar : 1;
private:
	/** Stored state of polys in case the brush state needs to be restroed */
	UPROPERTY()
	TObjectPtr<class UPolys> CachedPolys;

public:

	/** @return		The modifier's description string. */
	const FText& GetModifierDescription() const;

	/** @return		The modifier's tooltip string. */
	const FText& GetModifierTooltip() const;

	/** @return		true if the key was handled by this editor mode tool. */
	virtual bool InputKey(class FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event);

	/** @return		true if the delta was handled by this editor mode tool. */
	virtual bool InputDelta(class FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale);

	/**
	 * Drawing functions to allow modifiers to have better control over the screen.
	 */
	virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI);
	virtual void DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas);

	/**
	 * Applies the modifier.  Does nothing if the editor is not in geometry mode.
	 *
	 * @return		true if something happened.
	 */
	bool Apply();

	UE_DEPRECATED(4.25, "Supports has been deprected.  Use SupportsCurrentSelection instead")
	virtual bool Supports() { return SupportsCurrentSelection(); };

	/**
	 * @return		true if this modifier will work on the currently selected sub objects.
	 */
	virtual bool SupportsCurrentSelection(){return false;};

	/** Gives the individual modifiers a chance to do something the first time they are activated. */
	virtual void Initialize();
	
	/**
	 * Starts the modification of geometry data.
	 */
	bool StartModify();

	/**
	 * Ends the modification of geometry data.
	 */
	bool EndModify();

	/**
	 * Handles the starting of transactions against the selected ABrushes.
	 */
	void StartTrans();
	
	/**
	 * Handles the stopping of transactions against the selected ABrushes.
	 */
	void EndTrans();

	/**
	 * Store the current geom selections (Edge, Vert and Poly)
	 */
	void StoreCurrentGeomSelections( TArray<struct FGeomSelection>& SelectionArray, FGeomObjectPtr go );

	/**
	 * Store the current geom selections for all geom objects
	 */
	void StoreAllCurrentGeomSelections();

	virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime) {}

	/**
	 * Gives the modifier a chance to initialize it's internal state when activated.
	 */
	virtual void WasActivated() {}

	/**
	* Gives the modifier a chance to clean up when the user is switching away from it.
	*/
	virtual void WasDeactivated() {}
	
	/**
	 * Stores the current state of the brush so that upon faulty operations, the
	 * brush may be restored to its previous state
	 */
	 void CacheBrushState();
	 
	/**
	 * Restores the brush to its cached state
	 */
	 void RestoreBrushState();
	 
	/**
	 * @return		true if two edges in the shape overlap not at a vertex
	 */
	bool DoEdgesOverlap();

protected:
	/**
	 * Interface for displaying error messages.
	 *
	 * @param	InErrorMsg		The error message to display.
	 */
	void GeomError(const FString& InErrorMsg);
	
	/**
	 * Implements the modifier application.
	 */
	virtual bool OnApply();

	/**
	 * Updates the pivot offset of the selected brushes based on the current vertex positions
	 */
	virtual void UpdatePivotOffset();
};
