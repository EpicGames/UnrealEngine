// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Allows clipping of BSP brushes against a plane.
 */

#pragma once

#include "GeomModifier_Edit.h"
#include "GeomModifier_Clip.generated.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;

UCLASS()
class UGeomModifier_Clip : public UGeomModifier_Edit
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	uint32 bFlipNormal:1;

	UPROPERTY(EditAnywhere, Category=Settings)
	uint32 bSplit:1;

	/** The clip markers that the user has dropped down in the world so far. */
	UPROPERTY()
	TArray<FVector> ClipMarkers;

	/** The mouse position, in world space, where the user currently is hovering. */
	UPROPERTY()
	FVector SnappedMouseWorldSpacePos;


	//~ Begin UGeomModifier Interface
	virtual bool SupportsCurrentSelection() override;
	virtual bool InputKey(class FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas) override;
	virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime) override;
	virtual void WasActivated() override;
protected:
	virtual bool OnApply() override;
	//~ End UGeomModifier Interface

private:
	// @todo document
	void ApplyClip( bool InSplit, bool InFlipNormal );
};
