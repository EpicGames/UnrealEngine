// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Creates selected objects.
 */

#pragma once

#include "GeomModifier_Edit.h"
#include "GeomModifier_Create.generated.h"

UCLASS()
class UGeomModifier_Create : public UGeomModifier_Edit
{
	GENERATED_UCLASS_BODY()


	//~ Begin UGeomModifier Interface
	virtual bool SupportsCurrentSelection() override;
protected:
	virtual bool OnApply() override;
	//~ End UGeomModifier Interface
};
