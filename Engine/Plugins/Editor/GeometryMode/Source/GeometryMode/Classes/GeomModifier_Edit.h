// Copyright Epic Games, Inc. All Rights Reserved.


/**
 *
 */


//~=============================================================================
// GeomModifier_Edit: Maniupalating selected objects with the widget
//~=============================================================================

#pragma once

#include "GeomModifier.h"
#include "GeomModifier_Edit.generated.h"

class FViewport;

UCLASS(autoexpandcategories=Settings)
class UGeomModifier_Edit : public UGeomModifier
{
	GENERATED_UCLASS_BODY()

	virtual bool SupportsCurrentSelection() override;

	//~ Begin UGeomModifier Interface
	virtual bool InputDelta(class FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale) override;	
	//~ End UGeomModifier Interface
};
