// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Extrudes selected objects.
 */

#pragma once

#include "GeomModifier_Edit.h"
#include "GeomModifier_Extrude.generated.h"

class FViewport;

UCLASS()
class UGeomModifier_Extrude : public UGeomModifier_Edit
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	int32 Length;

	UPROPERTY(EditAnywhere, Category=Settings)
	int32 Segments;

	UPROPERTY()
	int32 SaveCoordSystem;


	//~ Begin UGeomModifier Interface
	virtual bool InputDelta(class FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool SupportsCurrentSelection() override;
	virtual void Initialize() override;
	virtual void WasActivated() override;
	virtual void WasDeactivated() override;

	/* Check the coordinates mode is local and warn the user with a suppressible dialog if it is not */
	void CheckCoordinatesMode();

protected:
	virtual bool OnApply() override;
	//~ End UGeomModifier Interface
private:
	void Apply(int32 InLength, int32 InSegments);
};
