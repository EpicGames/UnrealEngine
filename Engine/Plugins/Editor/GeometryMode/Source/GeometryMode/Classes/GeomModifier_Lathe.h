// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Revolves selected polygons around the pivot point.
 */

#pragma once

#include "GeomModifier_Edit.h"
#include "GeomModifier_Lathe.generated.h"

UCLASS()
class UGeomModifier_Lathe : public UGeomModifier_Edit
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	int32 TotalSegments;

	UPROPERTY(EditAnywhere, Category=Settings)
	int32 Segments;

	UPROPERTY(EditAnywhere, Category=Settings)
	uint32 AlignToSide:1;

	/** The axis of rotation to use when creating the brush.  This is automatically determined from the current ortho viewport. */
	UPROPERTY()
	TEnumAsByte<EAxis::Type> Axis;


	//~ Begin UGeomModifier Interface
	virtual bool SupportsCurrentSelection() override;
	virtual void Initialize() override;
protected:
	virtual bool OnApply() override;
	//~ End UGeomModifier Interface
private:
	void Apply( int32 InTotalSegments, int32 InSegments, EAxis::Type InAxis );
};
