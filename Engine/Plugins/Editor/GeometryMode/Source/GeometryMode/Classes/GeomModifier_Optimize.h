// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Optimizes selected objects by attempting to merge their polygons back together.
 */

#pragma once

#include "GeomModifier_Triangulate.h"
#include "GeomModifier_Optimize.generated.h"

UCLASS()
class UGeomModifier_Optimize : public UGeomModifier_Triangulate
{
	GENERATED_UCLASS_BODY()


	//~ Begin UGeomModifier Interface
protected:
	virtual bool OnApply() override;
	//~ End UGeomModifier Interface
};
