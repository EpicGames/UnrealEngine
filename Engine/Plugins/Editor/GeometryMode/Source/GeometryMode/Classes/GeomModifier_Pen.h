// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Allows the user to place verts in an orthographic viewport and create a brush afterwards.
 */

#pragma once

#include "GeomModifier_Edit.h"
#include "GeomModifier_Pen.generated.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;

UCLASS()
class UGeomModifier_Pen : public UGeomModifier_Edit
{
	GENERATED_UCLASS_BODY()

	/** If true, the shape will be automatically extruded into a brush upon completion. */
	UPROPERTY(EditAnywhere, Category=Settings)
	uint32 bAutoExtrude:1;

	/** If true, the tool will try and optimize the resulting triangles into convex polygons before creating the brush. */
	UPROPERTY(EditAnywhere, Category=Settings)
	uint32 bCreateConvexPolygons:1;

	/** If true, the resulting shape will be turned into an ABrushShape actor. */
	UPROPERTY(EditAnywhere, Category=Settings)
	uint32 bCreateBrushShape:1;

	/** How far to extrude the newly created brush if bAutoExtrude is set to true. */
	UPROPERTY(EditAnywhere, Category=Settings)
	int32 ExtrudeDepth;

	/** The vertices that the user has dropped down in the world so far. */
	UPROPERTY()
	TArray<FVector> ShapeVertices;

	/** The mouse position, in world space, where the user currently is hovering (snapped to grid if that setting is enabled). */
	UPROPERTY(transient)
	FVector MouseWorldSpacePos;


	FEditorViewportClient*		UsingViewportClient;

	//~ Begin UGeomModifier Interface
	virtual bool InputKey(class FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas) override;
	virtual bool SupportsCurrentSelection() override;
	virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime) override;
	virtual void WasActivated() override;
protected:
	virtual bool OnApply() override;
	//~ End UGeomModifier Interface
private:
	void Apply();
};
