// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryMode : ModuleRules
{
	public GeometryMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"BSPUtils",
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"RenderCore",
				"LevelEditor",
				"NavigationSystem",
				"EditorSubsystem",
				"Projects",
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
            new string[] {
				"PropertyEditor",
			}
        );
	}
}
