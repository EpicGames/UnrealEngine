// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrushEditingSubsystemImpl.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorGeometry.h"
#include "EditorModeManager.h"
#include "Engine/Brush.h"
#include "Engine/HitResult.h"
#include "GeometryEdMode.h"
#include "LevelEditorViewport.h"
#include "SnappingUtils.h"
#include "GeometryModeModule.h"
#include "ScopedTransaction.h"
#include "ViewportSelectionUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushEditingSubsystemImpl)

DEFINE_LOG_CATEGORY(LogBrushEditing);



/**
 * Utility method used by ClickGeomEdge and ClickGeomVertex.  Returns true if the projections of
 * the vectors onto the specified viewport plane are equal within the given tolerance.
 */
bool OrthoEqual(ELevelViewportType ViewportType, const FVector& Vec0, const FVector& Vec1, float Tolerance = 0.1f)
{
	bool bResult = false;
	switch (ViewportType)
	{
	case LVT_OrthoXY:
	case LVT_OrthoNegativeXY:
		bResult = FMath::Abs(Vec0.X - Vec1.X) < Tolerance && FMath::Abs(Vec0.Y - Vec1.Y) < Tolerance;
		break;
	case LVT_OrthoXZ:
	case LVT_OrthoNegativeXZ:
		bResult = FMath::Abs(Vec0.X - Vec1.X) < Tolerance && FMath::Abs(Vec0.Z - Vec1.Z) < Tolerance;
		break;
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeYZ:
		bResult = FMath::Abs(Vec0.Y - Vec1.Y) < Tolerance && FMath::Abs(Vec0.Z - Vec1.Z) < Tolerance;
		break;
	default:
		check(0);
		break;
	}
	return bResult;
}

UBrushEditingSubsystemImpl::UBrushEditingSubsystemImpl()
{

}

void UBrushEditingSubsystemImpl::Initialize(FSubsystemCollectionBase& Collection)
{

}

bool UBrushEditingSubsystemImpl::ProcessClickOnBrushGeometry(
	FEditorViewportClient* ViewportClient, HHitProxy* InHitProxy, const FViewportClick& Click
)
{
	bool bHandled = false;
	if (InHitProxy->IsA(HGeomPolyProxy::StaticGetType()))
	{
		HGeomPolyProxy* GeomHitProxy = (HGeomPolyProxy*)InHitProxy;
		if (GeomHitProxy->GetGeomObject())
		{
			FHitResult CheckResult(ForceInit);
			FCollisionQueryParams BoxParams(SCENE_QUERY_STAT(ProcessClickTrace), false, GeomHitProxy->GetGeomObject()->ActualBrush);
			bool bHit = ViewportClient->GetWorld()->SweepSingleByObjectType(CheckResult, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * HALF_WORLD_MAX, FQuat::Identity, FCollisionObjectQueryParams(ECC_WorldStatic), FCollisionShape::MakeBox(FVector(1.f)), BoxParams);

			if (bHit)
			{
				GEditor->UnsnappedClickLocation = CheckResult.Location;
				GEditor->ClickLocation = CheckResult.Location;
				GEditor->ClickPlane = FPlane(CheckResult.Location, CheckResult.Normal);
			}

			if (!UE::Editor::ViewportSelectionUtilities::ClickActor(ViewportClient, GeomHitProxy->GetGeomObject()->ActualBrush, Click, false))
			{
				ProcessClickOnGeomPoly(ViewportClient, GeomHitProxy, Click);
			}

			ViewportClient->Invalidate(true, true);
		}

		bHandled = true;
	}
	else if (InHitProxy->IsA(HGeomEdgeProxy::StaticGetType()))
	{
		HGeomEdgeProxy* GeomHitProxy = (HGeomEdgeProxy*)InHitProxy;

		if (GeomHitProxy->GetGeomObject() != nullptr)
		{
			if (!ProcessClickOnGeomEdge(ViewportClient, GeomHitProxy, Click))
			{
				UE::Editor::ViewportSelectionUtilities::ClickActor(ViewportClient, GeomHitProxy->GetGeomObject()->ActualBrush, Click, true);
			}
		}

		bHandled = true;
	}
	else if (InHitProxy->IsA(HGeomVertexProxy::StaticGetType()))
	{
		ProcessClickOnGeomVertex(ViewportClient, (HGeomVertexProxy*)InHitProxy, Click);

		bHandled = true;
	}

	return bHandled;
}

bool UBrushEditingSubsystemImpl::ProcessClickOnGeomPoly(FEditorViewportClient* ViewportClient, HGeomPolyProxy* GeomHitProxy, const FViewportClick& Click)
{
	if (GeomHitProxy == NULL)
	{
		UE_LOGF(LogBrushEditing, Warning, "Invalid hitproxy");
		return false;
	}

	if (!GeomHitProxy->GeomObjectWeakPtr.IsValid())
	{
		return false;
	}

	// Pivot snapping
	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);

		return true;
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && Click.IsControlDown() && Click.IsShiftDown() && !Click.IsAltDown())
	{
		GEditor->SelectActor(GeomHitProxy->GetGeomObject()->GetActualBrush(), false, true);
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton)
	{
		// This should only happen in geometry mode
		FEdMode* Mode = GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
		if (Mode)
		{
			if ((GeomHitProxy->GetGeomObject() != NULL) && (GeomHitProxy->GetGeomObject()->PolyPool.IsValidIndex(GeomHitProxy->PolyIndex) == true))
			{
				Mode->GetCurrentTool()->StartTrans();

				if (!Click.IsControlDown())
				{
					Mode->GetCurrentTool()->SelectNone();
				}

				FGeomPoly& gp = GeomHitProxy->GetGeomObject()->PolyPool[GeomHitProxy->PolyIndex];
				gp.Select(Click.IsControlDown() ? !gp.IsSelected() : true);

				Mode->SelectionChanged();

				Mode->GetCurrentTool()->EndTrans();
				ViewportClient->Invalidate(true, false);
			}
			else
			{
				//try to get the name of the object also
				FString name = TEXT("UNKNOWN");
				if (GeomHitProxy->GetGeomObject()->GetActualBrush() != NULL)
				{
					name = GeomHitProxy->GetGeomObject()->GetActualBrush()->GetName();
				}
				UE_LOGF(LogBrushEditing, Warning, "Invalid PolyIndex %d on %ls", GeomHitProxy->PolyIndex, *name);
			}
		}
	}

	return false;
}

bool UBrushEditingSubsystemImpl::ProcessClickOnGeomEdge(FEditorViewportClient* ViewportClient, HGeomEdgeProxy* GeomHitProxy, const FViewportClick& Click)
{
	// Pivot snapping
	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);

		return true;
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && Click.IsControlDown() && Click.IsShiftDown() && !Click.IsAltDown())
	{
		GEditor->SelectActor(GeomHitProxy->GetGeomObject()->GetActualBrush(), false, true);

		return true;
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton)
	{
		FEdMode* Mode = GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

		if (Mode)
		{
			Mode->GetCurrentTool()->StartTrans();

			const bool bControlDown = Click.IsControlDown();
			if (!bControlDown)
			{
				Mode->GetCurrentTool()->SelectNone();
			}

			FGeomEdge& HitEdge = GeomHitProxy->GetGeomObject()->EdgePool[GeomHitProxy->EdgeIndex];
			HitEdge.Select(bControlDown ? !HitEdge.IsSelected() : true);

			if (ViewportClient->IsOrtho())
			{
				// Select all edges in the brush that match the projected mid point of the original edge.
				for (int32 EdgeIndex = 0; EdgeIndex < GeomHitProxy->GetGeomObject()->EdgePool.Num(); ++EdgeIndex)
				{
					if (EdgeIndex != GeomHitProxy->EdgeIndex)
					{
						FGeomEdge& GeomEdge = GeomHitProxy->GetGeomObject()->EdgePool[EdgeIndex];
						if (OrthoEqual(ViewportClient->ViewportType, GeomEdge.GetMid(), HitEdge.GetMid()))
						{
							GeomEdge.Select(bControlDown ? !GeomEdge.IsSelected() : true);
						}
					}
				}
			}

			Mode->SelectionChanged();

			Mode->GetCurrentTool()->EndTrans();
			ViewportClient->Invalidate(true, true);
			return true;
		}

		return false;


	}

	return false;
}

bool UBrushEditingSubsystemImpl::ProcessClickOnGeomVertex(FEditorViewportClient* ViewportClient, HGeomVertexProxy* GeomHitProxy, const FViewportClick& Click)
{
	if (GeomHitProxy->GetGeomObject() == nullptr)
	{
		return false;
	}

	if (!GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry))
	{
		return false;
	}

	FEdModeGeometry* Mode = static_cast<FEdModeGeometry*>(GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry));

	// Note: The expected behavior is that right clicking on a vertex will snap the vertex that was
	// right-clicked on to the nearest grid point, then move all SELECTED verts by the appropriate
	// delta.  So, we need to handle the right mouse button click BEFORE we change the selection set below.

	if (Click.GetKey() == EKeys::RightMouseButton)
	{

		if (GeomHitProxy->VertexIndex < 0 || GeomHitProxy->VertexIndex >= GeomHitProxy->GetGeomObject()->VertexPool.Num())
		{
			UE_LOGF(LogBrushEditing, Warning, "Invalid InHitProxy->VertexIndex");
			return false;
		}

		FModeTool_GeometryModify* Tool = static_cast<FModeTool_GeometryModify*>(Mode->GetCurrentTool());
		Tool->StartTrans();

		// Compute out far to move to get back on the grid.
		const FVector WorldLoc = GeomHitProxy->GetGeomObject()->GetActualBrush()->ActorToWorld().TransformPosition((FVector)GeomHitProxy->GetGeomObject()->VertexPool[GeomHitProxy->VertexIndex]);

		FVector SnappedLoc = WorldLoc;
		FSnappingUtils::SnapPointToGrid(SnappedLoc, FVector(GEditor->GetGridSize()));

		const FVector Delta = SnappedLoc - WorldLoc;
		GEditor->SetPivot(SnappedLoc, false, false);

		for (int32 VertexIndex = 0; VertexIndex < GeomHitProxy->GetGeomObject()->VertexPool.Num(); ++VertexIndex)
		{
			FGeomVertex& GeomVertex = GeomHitProxy->GetGeomObject()->VertexPool[VertexIndex];
			if (GeomVertex.IsSelected())
			{
				GeomVertex += (FVector3f)Delta;
			}
		}

		Tool->EndTrans();
		GeomHitProxy->GetGeomObject()->SendToSource();
		ViewportClient->Invalidate(true, true);

		// HACK: The Bsp update has to occur after SendToSource() updates the vert pool, putting it outside
		// of the mode tool's transaction, therefore, the Bsp update requires a transaction of its own
		{
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GeoModeVertexSnap", "Vertex Snap"));

			// Update Bsp
			GEditor->RebuildAlteredBSP();
		}
	}

	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		// Pivot snapping

		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);

		return true;
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && Click.IsControlDown() && Click.IsShiftDown() && !Click.IsAltDown())
	{
		GEditor->SelectActor(GeomHitProxy->GetGeomObject()->GetActualBrush(), false, true);
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton)
	{
		Mode->GetCurrentTool()->StartTrans();

		// Disable Ctrl+clicking for selection if selecting with RMB.
		const bool bControlDown = Click.IsControlDown();
		if (!bControlDown)
		{
			Mode->GetCurrentTool()->SelectNone();
		}

		FGeomVertex& HitVertex = GeomHitProxy->GetGeomObject()->VertexPool[GeomHitProxy->VertexIndex];
		bool bSelect = bControlDown ? !HitVertex.IsSelected() : true;

		HitVertex.Select(bSelect);

		if (ViewportClient->IsOrtho())
		{
			// Select all vertices that project to the same location.
			for (int32 VertexIndex = 0; VertexIndex < GeomHitProxy->GetGeomObject()->VertexPool.Num(); ++VertexIndex)
			{
				if (VertexIndex != GeomHitProxy->VertexIndex)
				{
					FGeomVertex& GeomVertex = GeomHitProxy->GetGeomObject()->VertexPool[VertexIndex];
					if (OrthoEqual(ViewportClient->ViewportType, (FVector)GeomVertex, (FVector)HitVertex))
					{
						GeomVertex.Select(bSelect);
					}
				}
			}
		}

		Mode->SelectionChanged();

		Mode->GetCurrentTool()->EndTrans();

		ViewportClient->Invalidate(true, true);

		return true;
	}

	return false;
}

void UBrushEditingSubsystemImpl::UpdateGeometryFromSelectedBrushes()
{
	if (IsGeometryEditorModeActive())
	{
		// If we are in geometry mode, make sure to update the mode with new source data for selected brushes
		FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
		Mode->GetFromSource();
	}

}

void UBrushEditingSubsystemImpl::UpdateGeometryFromBrush(ABrush* Brush)
{
	// Determine if we are in geometry edit mode.
	if (IsGeometryEditorModeActive())
	{
		// If we are in geometry mode, go through the list of geometry objects
		// and find our current brush and update its source data as it might have changed 
		// in RecomputePoly
		FEdModeGeometry* GeomMode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
		FEdModeGeometry::TGeomObjectIterator GeomModeIt = GeomMode->GeomObjectItor();
		for (; GeomModeIt; ++GeomModeIt)
		{
			FGeomObjectPtr Object = *GeomModeIt;
			if (Object->GetActualBrush() == Brush)
			{
				// We found our current brush, update the geometry object's data
				Object->GetFromSource();
				break;
			}
		}
	}
}

bool UBrushEditingSubsystemImpl::IsGeometryEditorModeActive() const
{
	return GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry);
}

void UBrushEditingSubsystemImpl::DeselectAllEditingGeometry()
{
	if(IsGeometryEditorModeActive())
	{
		FEdModeGeometry* Mode = GLevelEditorModeTools().GetActiveModeTyped<FEdModeGeometry>(FGeometryEditingModes::EM_Geometry);
		if (Mode)
		{
			Mode->GeometrySelectNone(true, true);
		}
	}
}

bool UBrushEditingSubsystemImpl::HandleActorDelete()
{
	bool bHandled = false;
	// If geometry mode is active, give it a chance to handle this command.  If it does not, use the default handler
	if (IsGeometryEditorModeActive())
	{
		bHandled = ((FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry))->ExecDelete();
	}

	return bHandled;
}

