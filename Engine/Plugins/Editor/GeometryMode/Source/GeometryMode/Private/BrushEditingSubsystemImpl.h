// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/BrushEditingSubsystem.h"

#include "BrushEditingSubsystemImpl.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBrushEditing, Log, All);

struct HGeomPolyProxy;
struct HGeomEdgeProxy;
struct HGeomVertexProxy;

UCLASS()
class UBrushEditingSubsystemImpl : public UBrushEditingSubsystem
{
	GENERATED_BODY()

public:
	UBrushEditingSubsystemImpl();

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	virtual bool ProcessClickOnBrushGeometry(FEditorViewportClient* ViewportClient, HHitProxy* InHitProxy, const FViewportClick& Click) override;

	virtual void UpdateGeometryFromSelectedBrushes() override;
	virtual void UpdateGeometryFromBrush(ABrush* Brush) override;

	virtual bool IsGeometryEditorModeActive() const override;

	virtual void DeselectAllEditingGeometry() override;

	virtual bool HandleActorDelete() override;
private:

	bool ProcessClickOnGeomPoly(FEditorViewportClient* ViewportClient, HGeomPolyProxy* GeomHitProxy, const FViewportClick& Click);
	bool ProcessClickOnGeomEdge(FEditorViewportClient* ViewportClient, HGeomEdgeProxy* GeomHitProxy, const FViewportClick& Click);
	bool ProcessClickOnGeomVertex(FEditorViewportClient* ViewportClient, HGeomVertexProxy* GeomHitProxy, const FViewportClick& Click);

};
