// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorGeometry.h"
#include "Engine/Brush.h"
#include "Engine/Polys.h"
#include "EditorModeManager.h"
#include "GeometryEdMode.h"
#include "GeometryModeModule.h"
#include "Model.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorGeometry, Log, All);


// Doxygen cannot parse these correctly since the declarations are made in Editor, not UnrealEd
#if !UE_BUILD_DOCS
IMPLEMENT_HIT_PROXY(HGeomPolyProxy, HHitProxy);
IMPLEMENT_HIT_PROXY(HGeomEdgeProxy, HHitProxy);
IMPLEMENT_HIT_PROXY(HGeomVertexProxy, HHitProxy);
#endif



///////////////////////////////////////////////////////////////////////////////
//
// FGeomBase
//
///////////////////////////////////////////////////////////////////////////////

FGeomBase::FGeomBase()
{
	SelectionIndex = INDEX_NONE;
	Normal = FVector::ZeroVector;
	ParentObjectIndex = INDEX_NONE;
}

void FGeomBase::Select( bool InSelect )
{
	FEditorModeTools& Tools = GLevelEditorModeTools();
	check(Tools.IsModeActive(FGeometryEditingModes::EM_Geometry));

	if (InSelect)
	{
		SelectionIndex = GetParentObject()->GetNewSelectionIndex();
	}
	else
	{
		SelectionIndex = INDEX_NONE;
	}

	// If something is selected, move the pivot and snap locations to the widget location.
	if( IsSelected() )
	{
		Tools.SetPivotLocation( GetWidgetLocation(), false );
	}

	GetParentObject()->DirtySelectionOrder();
}

FGeomObjectPtr FGeomBase::GetParentObject()
{
	check( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) );
	check( ParentObjectIndex > INDEX_NONE );

	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return mode->GetGeomObject( ParentObjectIndex );
}

const FGeomObjectPtr FGeomBase::GetParentObject() const
{
	check( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) );
	check( ParentObjectIndex > INDEX_NONE );

	const FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return mode->GetGeomObject( ParentObjectIndex );
}

int32 FGeomObject::SetPivotFromSelectionArray( TArray<struct FGeomSelection>& SelectionArray )
{
	int32 HighestSelectionIndex = INDEX_NONE;
	int32 ArrayIndex = 0;
	//find 'highest' selection
	for( int32 s = 0 ; s < SelectionArray.Num() ; ++s )
	{
		FGeomSelection* gs = &SelectionArray[s];

		if( gs->SelectionIndex > HighestSelectionIndex )
		{
			HighestSelectionIndex = gs->SelectionIndex;
			ArrayIndex = s;
		}
	}
	//set the pivot if we found a valid selection
	if( HighestSelectionIndex != -1 )
	{
		FGeomSelection* gs = &SelectionArray[ ArrayIndex ];
		FEditorModeTools& Tools = GLevelEditorModeTools();
		switch( gs->Type )
		{
		case GS_Poly:
			PolyPool[ gs->Index ].ForceSelectionIndex( gs->SelectionIndex );
			Tools.SetPivotLocation( PolyPool[ gs->Index ].GetWidgetLocation(), false );
			break;

		case GS_Edge:
			EdgePool[ gs->Index ].ForceSelectionIndex( gs->SelectionIndex );
			Tools.SetPivotLocation( EdgePool[ gs->Index ].GetWidgetLocation(), false );
			break;

		case GS_Vertex:
			VertexPool[ gs->Index ].ForceSelectionIndex( gs->SelectionIndex );
			Tools.SetPivotLocation( VertexPool[ gs->Index ].GetWidgetLocation(), false );
			break;
		}
	}
	return HighestSelectionIndex;
}

void FGeomObject::UpdateFromSelectionArray(TArray<struct FGeomSelection>& SelectionArray)
{
	FEditorModeTools& Tools = GLevelEditorModeTools();

	for (const auto& Selection : SelectionArray)
	{
		switch (Selection.Type)
		{
		case GS_Poly:
			PolyPool[Selection.Index].ForceSelectionIndex(Selection.SelectionIndex);
			break;

		case GS_Edge:
			EdgePool[Selection.Index].ForceSelectionIndex(Selection.SelectionIndex);
			break;

		case GS_Vertex:
			VertexPool[Selection.Index].ForceSelectionIndex(Selection.SelectionIndex);
			break;
		}
	}

	DirtySelectionOrder();
	CompileSelectionOrder();
}

///////////////////////////////////////////////////////////////////////////////
//
// FGeomVertex
//
///////////////////////////////////////////////////////////////////////////////

FGeomVertex::FGeomVertex()
{
	X = Y = Z = 0;
	ParentObjectIndex = INDEX_NONE;
}

FVector FGeomVertex::GetWidgetLocation()
{
	return GetParentObject()->GetActualBrush()->ActorToWorld().TransformPosition((FVector)*this );
}

FVector FGeomVertex::GetMidPoint() const
{
	return (FVector)*this;
}

FVector3f* FGeomVertex::GetActualVertex( FPolyVertexIndex& InPVI )
{
	return &(GetParentObject()->GetActualBrush()->Brush->Polys->Element[ InPVI.PolyIndex ].Vertices[ InPVI.VertexIndex ]);
}

FRotator FGeomVertex::GetWidgetRotation() const
{
	return FRotator::ZeroRotator;
}

///////////////////////////////////////////////////////////////////////////////
//
// FGeomEdge
//
///////////////////////////////////////////////////////////////////////////////

FGeomEdge::FGeomEdge()
{
	ParentObjectIndex = INDEX_NONE;
	VertexIndices[0] = INDEX_NONE;
	VertexIndices[1] = INDEX_NONE;
}

// Returns true if InEdge matches this edge, independant of winding.
bool FGeomEdge::IsSameEdge( const FGeomEdge& InEdge ) const
{
	return (( VertexIndices[0] == InEdge.VertexIndices[0] && VertexIndices[1] == InEdge.VertexIndices[1] ) ||
			( VertexIndices[0] == InEdge.VertexIndices[1] && VertexIndices[1] == InEdge.VertexIndices[0] ));
}

FVector FGeomEdge::GetWidgetLocation()
{
	FVector dir = ((FVector)GetParentObject()->VertexPool[ VertexIndices[1] ] - (FVector)GetParentObject()->VertexPool[ VertexIndices[0] ]);
	const double dist = dir.Size() / 2;
	dir.Normalize();
	const FVector loc = (FVector)GetParentObject()->VertexPool[ VertexIndices[0] ] + (dir * dist);
	return GetParentObject()->GetActualBrush()->ActorToWorld().TransformPosition( loc );
}

FVector FGeomEdge::GetMidPoint() const
{
	const FGeomVertex* wk0 = &(GetParentObject()->VertexPool[ VertexIndices[0] ]);
	const FGeomVertex* wk1 = &(GetParentObject()->VertexPool[ VertexIndices[1] ]);

	const FVector v0( wk0->X, wk0->Y, wk0->Z );
	const FVector v1( wk1->X, wk1->Y, wk1->Z );

	return (v0 + v1) / 2;
}

///////////////////////////////////////////////////////////////////////////////
//
// FGeomPoly
//
///////////////////////////////////////////////////////////////////////////////

FGeomPoly::FGeomPoly()
{
	ParentObjectIndex = INDEX_NONE;
}

FVector FGeomPoly::GetWidgetLocation()
{
	const FPoly* Poly = GetActualPoly();
	FVector Wk(0.f,0.f,0.f);
	if ( Poly->Vertices.Num() > 0 )
	{
		for( int32 VertIndex = 0 ; VertIndex < Poly->Vertices.Num() ; ++VertIndex )
		{
			Wk += (FVector)Poly->Vertices[VertIndex];
		}
		Wk = Wk / Poly->Vertices.Num();
	}

	return GetParentObject()->GetActualBrush()->ActorToWorld().TransformPosition( Wk );
}

FVector FGeomPoly::GetMidPoint() const
{
	FVector Wk(0,0,0);
	int32 Count = 0;

	for( int32 e = 0 ; e < EdgeIndices.Num() ; ++e )
	{
		const FGeomEdge* ge = &GetParentObject()->EdgePool[ EdgeIndices[e] ];
		Wk += (FVector)GetParentObject()->VertexPool[ ge->VertexIndices[0] ];
		Count++;
		Wk += (FVector)GetParentObject()->VertexPool[ ge->VertexIndices[1] ];
		Count++;
	}

	check( Count );
	return Wk / Count;
}

FPoly* FGeomPoly::GetActualPoly()
{
	FGeomObjectPtr Parent = GetParentObject();
	check( Parent.IsValid() );
	ABrush* Brush = Parent->GetActualBrush();
	check( Brush );

	return &( Brush->Brush->Polys->Element[ActualPolyIndex] );
}

///////////////////////////////////////////////////////////////////////////////
//
// FGeomObject
//
///////////////////////////////////////////////////////////////////////////////

FGeomObject::FGeomObject():
	LastSelectionIndex( INDEX_NONE )
{
	DirtySelectionOrder();
	ActualBrush = NULL;
}

FVector FGeomObject::GetWidgetLocation()
{
	return GetActualBrush()->GetActorLocation();
}

int32 FGeomObject::AddVertexToPool( int32 InObjectIndex, int32 InParentPolyIndex, int32 InPolyIndex, int32 InVertexIndex )
{
	FGeomVertex* gv = NULL;
	FVector3f CmpVtx = GetActualBrush()->Brush->Polys->Element[ InPolyIndex ].Vertices[InVertexIndex];

	// See if the vertex is already in the pool
	for( int32 x = 0 ; x < VertexPool.Num() ; ++x )
	{
		if( FVector3f::PointsAreNear( VertexPool[x], CmpVtx, 0.5f ) )
		{
			gv = &VertexPool[x];
			gv->ActualVertexIndices.AddUnique( FPolyVertexIndex( InPolyIndex, InVertexIndex ) );
			gv->ParentPolyIndices.AddUnique( InParentPolyIndex );
			return x;
		}
	}

	// If not, add it...
	if( gv == NULL )
	{
		new( VertexPool )FGeomVertex();
		gv = &VertexPool[ VertexPool.Num()-1 ];
		*gv = CmpVtx;
	}

	gv->ActualVertexIndices.AddUnique( FPolyVertexIndex( InPolyIndex, InVertexIndex ) );
	gv->SetParentObjectIndex( InObjectIndex );
	gv->ParentPolyIndices.AddUnique( InParentPolyIndex );

	return VertexPool.Num()-1;
}

int32 FGeomObject::AddEdgeToPool( FGeomPoly* InPoly, int32 InParentPolyIndex, int32 InVectorIdxA, int32 InVectorIdxB )
{
	const int32 idx0 = AddVertexToPool( InPoly->GetParentObjectIndex(), InParentPolyIndex, InPoly->ActualPolyIndex, InVectorIdxA );
	const int32 idx1 = AddVertexToPool( InPoly->GetParentObjectIndex(), InParentPolyIndex, InPoly->ActualPolyIndex, InVectorIdxB );

	// See if the edge is already in the pool.  If so, leave.
	FGeomEdge* ge = NULL;
	for( int32 e = 0 ; e < EdgePool.Num() ; ++e )
	{
		if( EdgePool[e].VertexIndices[0] == idx0 && EdgePool[e].VertexIndices[1] == idx1 )
		{
			EdgePool[e].ParentPolyIndices.Add( InPoly->GetParentObject()->PolyPool.Find( *InPoly ) );
			return e;
		}
	}

	// Add a new edge to the pool and set it up.
	new( EdgePool )FGeomEdge();
	ge = &EdgePool[ EdgePool.Num()-1 ];

	ge->VertexIndices[0] = idx0;
	ge->VertexIndices[1] = idx1;

	ge->ParentPolyIndices.Add( InPoly->GetParentObject()->PolyPool.Find( *InPoly ) );
	ge->SetParentObjectIndex( InPoly->GetParentObjectIndex() );

	return EdgePool.Num()-1;
}

/**
 * Removes all geometry data and reconstructs it from the source brushes.
 */

void FGeomObject::GetFromSource()
{
	PolyPool.Empty();
	EdgePool.Empty();
	VertexPool.Empty();

	for( int32 p = 0 ; p < GetActualBrush()->Brush->Polys->Element.Num() ; ++p )
	{
		FPoly* poly = &(GetActualBrush()->Brush->Polys->Element[p]);

		new( PolyPool )FGeomPoly();
		FGeomPoly* gp = &PolyPool[ PolyPool.Num()-1 ];
		gp->SetParentObjectIndex( GetObjectIndex() );
		gp->ActualPolyIndex = p;

		for( int32 v = 1 ; v <= poly->Vertices.Num() ; ++v )
		{
			int32 idx = (v == poly->Vertices.Num()) ? 0 : v,
				previdx = v-1;

			int32 eidx = AddEdgeToPool( gp, PolyPool.Num()-1, previdx, idx );
			gp->EdgeIndices.Add( eidx );
		}
	}

	ComputeData();
}

int32 FGeomObject::GetObjectIndex()
{
	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return INDEX_NONE;
	}

	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		if( (*Itor).Get() == this )
		{
			return Itor.GetIndex();
		}
	}

	check(0);	// Should never happen
	return INDEX_NONE;
}

/**
 * Sends the vertex data that we have back to the source vertices.
 */

void FGeomObject::SendToSource()
{
	for (int32 v = 0; v < VertexPool.Num(); ++v)
	{
		FGeomVertex* gv = &VertexPool[v];

		for (int32 x = 0; x < gv->ActualVertexIndices.Num(); ++x)
		{
			const TArray<FPoly>& Element = gv->GetParentObject()->GetActualBrush()->Brush->Polys->Element;
			FPolyVertexIndex& PVI = gv->ActualVertexIndices[x];
			if (ensure(PVI.PolyIndex < Element.Num() && PVI.VertexIndex < Element[PVI.PolyIndex].Vertices.Num()))
			{
				FVector3f* vtx = gv->GetActualVertex(PVI);
				vtx->X = gv->X;
				vtx->Y = gv->Y;
				vtx->Z = gv->Z;
			}
		}
	}
}

/**
 * Finalizes the source geometry by checking for invalid polygons,
 * updating components, etc. - anything that needs to be done
 * before the engine will accept the resulting brushes/polygons
 * as valid.
 */

bool FGeomObject::FinalizeSourceData()
{
	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return 0;
	}

	ABrush* brush = GetActualBrush();
	bool Ret = false;
	double StartTime = FPlatformTime::Seconds();
	const double TimeLimit = 10.0;

	// Remove invalid polygons from the brush

	for( int32 x = 0 ; x < brush->Brush->Polys->Element.Num() ; ++x )
	{
		FPoly* Poly = &brush->Brush->Polys->Element[x];

		if( Poly->Vertices.Num() < 3 )
		{
			brush->Brush->Polys->Element.RemoveAt( x );
			x = -1;
		}
	}

	for( int32 p = 0 ; p < brush->Brush->Polys->Element.Num() ; ++p )
	{
		FPoly* Poly = &(brush->Brush->Polys->Element[p]);
		Poly->iLink = p;
		int32 SaveNumVertices = Poly->Vertices.Num();

		const bool bTimeLimitExpired = TimeLimit < FPlatformTime::Seconds() - StartTime;

		if( !Poly->IsCoplanar() || !Poly->IsConvex() )
		{
			// If the polygon is no longer coplanar and/or convex, break it up into separate triangles.

			FPoly WkPoly = *Poly;
			brush->Brush->Polys->Element.RemoveAt( p );

			TArray<FPoly> Polygons;
			if( !bTimeLimitExpired && WkPoly.Triangulate( brush, Polygons ) > 0 )
			{
				FPoly::OptimizeIntoConvexPolys( brush, Polygons );

				for( int32 t = 0 ; t < Polygons.Num() ; ++t )
				{
					brush->Brush->Polys->Element.Add( Polygons[t] );
				}
			}

			p = -1;
			Ret = true;
		}
		else
		{
			int32 FixResult = Poly->Fix();
			if( FixResult != SaveNumVertices )
			{
				// If the polygon collapses after running "Fix" against it, it needs to be
				// removed from the brushes polygon list.

				if( bTimeLimitExpired || FixResult == 0 )
				{
					brush->Brush->Polys->Element.RemoveAt( p );
				}

				p = -1;
				Ret = true;
			}
			else
			{
				// If we get here, the polygon is valid and needs to be kept.  Finalize its internals.

				Poly->Finalize(brush,1);
			}
		}
	}

	if (TimeLimit < FPlatformTime::Seconds() - StartTime)
	{
		UE_LOGF(LogEditorGeometry, Error, "FGeomObject::FinalizeSourceData() failed because it took too long");
	}

	brush->ReregisterAllComponents();

	return Ret;
}

/**
 * Recomputes data specific to the geometry data (i.e. normals, mid points, etc)
 */

void FGeomObject::ComputeData()
{
	int32 Count;
	FVector Wk;

	// Polygons

	for( int32 p = 0 ; p < PolyPool.Num() ; ++p )
	{
		FGeomPoly* poly = &PolyPool[p];

		poly->SetNormal( (FVector)poly->GetActualPoly()->Normal );
		poly->SetMid( poly->GetMidPoint() );

	}

	// Vertices (= average normal of all the polygons that touch it)

	FGeomVertex* gv;
	for( int32 v = 0 ; v < VertexPool.Num() ; ++v )
	{
		gv = &VertexPool[v];
		Count = 0;
		Wk = FVector::ZeroVector;

		for( int32 e = 0 ; e < EdgePool.Num() ; ++e )
		{
			FGeomEdge* ge = &EdgePool[e];

			FGeomVertex *v0 = &VertexPool[ ge->VertexIndices[0] ],
				*v1 = &VertexPool[ ge->VertexIndices[1] ];

			if( gv == v0 || gv == v1 )
			{
				for( int32 p = 0 ; p < ge->ParentPolyIndices.Num() ; ++p )
				{
					FGeomPoly* poly = &PolyPool[ ge->ParentPolyIndices[p] ];

					Wk += poly->GetNormal();
					Count++;
				}
			}
		}

		gv->SetNormal( Wk / Count );
		gv->SetMid( gv->GetMidPoint() );
	}

	// Edges (= average normal of all the polygons that touch it)

	FGeomEdge* ge;
	for( int32 e = 0 ; e < EdgePool.Num() ; ++e )
	{
		ge = &EdgePool[e];
		Count = 0;
		Wk = FVector::ZeroVector;

		for( int32 e2 = 0 ; e2 < EdgePool.Num() ; ++e2 )
		{
			FGeomEdge* ge2 = &EdgePool[e2];

			if( ge->IsSameEdge( *ge2 ) )
			{
				for( int32 p = 0 ; p < ge2->ParentPolyIndices.Num(); ++p )
				{
					FGeomPoly* gp = &PolyPool[ ge2->ParentPolyIndices[p] ];

					Wk += (FVector)gp->GetActualPoly()->Normal;
					Count++;

				}
			}
		}

		ge->SetNormal( Wk / Count );
		ge->SetMid( ge->GetMidPoint() );
	}
}

void FGeomObject::ClearData()
{
	EdgePool.Empty();
	VertexPool.Empty();
}

FVector FGeomObject::GetMidPoint() const
{
	return GetActualBrush()->GetActorLocation();
}

int32 FGeomObject::GetNewSelectionIndex()
{
	LastSelectionIndex++;
	return LastSelectionIndex;
}

void FGeomObject::SelectNone()
{
	Select( false );

	for( int32 i = 0 ; i < EdgePool.Num() ; ++i )
	{
		EdgePool[i].Select( false );
	}
	for( int32 i = 0 ; i < PolyPool.Num() ; ++i )
	{
		PolyPool[i].Select( false );
	}
	for( int32 i = 0 ; i < VertexPool.Num() ; ++i )
	{
		VertexPool[i].Select( false );
	}

	LastSelectionIndex = INDEX_NONE;
}

/**
 * Compiles the selection order array by putting every geometry object
 * with a valid selection index into the array, and then sorting it.
 */
struct FCompareSelectionIndex
{
	FORCEINLINE bool operator()( const FGeomBase& A, const FGeomBase& B ) const
	{
		return (A.GetSelectionIndex() - B.GetSelectionIndex()) < 0;
	}
};

void FGeomObject::CompileSelectionOrder()
{
	// Only compile the array if it's dirty.

	if( !bSelectionOrderDirty )
	{
		return;
	}

	SelectionOrder.Empty();

	for( int32 i = 0 ; i < EdgePool.Num() ; ++i )
	{
		FGeomEdge* ge = &EdgePool[i];
		if( ge->GetSelectionIndex() > INDEX_NONE )
		{
			SelectionOrder.Add( ge );
		}
	}
	for( int32 i = 0 ; i < PolyPool.Num() ; ++i )
	{
		FGeomPoly* gp = &PolyPool[i];
		if( gp->GetSelectionIndex() > INDEX_NONE )
		{
			SelectionOrder.Add( gp );
		}
	}
	for( int i = 0 ; i < VertexPool.Num() ; ++i )
	{
		FGeomVertex* gv = &VertexPool[i];
		if( gv->GetSelectionIndex() > INDEX_NONE )
		{
			SelectionOrder.Add( gv );
		}
	}

	if( SelectionOrder.Num() )
	{
		SelectionOrder.Sort(FCompareSelectionIndex());
	}

	bSelectionOrderDirty = 0;
}

/**
 * Compiles a list of unique edges.  This runs through the edge pool
 * and only adds edges into the pool that aren't already there (the
 * difference being that this routine counts edges that share the same
 * vertices, but are wound backwards to each other, as being equal).
 *
 * @param	InEdges	The edge array to fill up
 */

void FGeomObject::CompileUniqueEdgeArray( TArray<FGeomEdge>* InEdges )
{
	InEdges->Empty();

	// Fill the array with any selected edges

	for( int32 e = 0 ; e < EdgePool.Num() ; ++e )
	{
		FGeomEdge* ge = &EdgePool[e];

		if( ge->IsSelected() )
		{
			InEdges->Add( *ge );
		}
	}

	// Gather up any other edges that share the same position

	for( int32 e = 0 ; e < EdgePool.Num() ; ++e )
	{
		FGeomEdge* ge = &EdgePool[e];

		// See if this edge is in the array already.  If so, add its parent
		// polygon indices into the transient edge arrays list.

		for( int32 x = 0 ; x < InEdges->Num() ; ++x )
		{
			FGeomEdge* geWk = &((*InEdges)[x]);

			if( geWk->IsSameEdge( *ge ) )
			{
				// The parent polygon indices of both edges must be combined so that the resulting
				// item in the edge array will point to the complete list of polygons that share that edge.

				for( int32 p = 0 ; p < ge->ParentPolyIndices.Num() ; ++p )
				{
					geWk->ParentPolyIndices.AddUnique( ge->ParentPolyIndices[p] );
				}

				break;
			}
		}
	}
}

void FGeomObject::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( ActualBrush );
}

FString FGeomObject::GetReferencerName() const
{
	return TEXT("FGeomObject");
}
