// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryEdMode.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Misc/FeedbackContext.h"
#include "SceneView.h"
#include "Selection.h"
#include "Settings/EditorStyleSettings.h"
#include "Materials/Material.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "TextureResource.h"
#include "Toolkits/ToolkitManager.h"
#include "BSPOps.h"
#include "GeometryMode.h"
#include "EditorGeometry.h"
#include "DynamicMeshBuilder.h"
#include "GeomModifier_Clip.h"
#include "GeomModifier_Create.h"
#include "GeomModifier_Delete.h"
#include "GeomModifier_Extrude.h"
#include "GeomModifier_Flip.h"
#include "GeomModifier_Lathe.h"
#include "GeomModifier_Pen.h"
#include "GeomModifier_Split.h"
#include "GeomModifier_Optimize.h"
#include "GeomModifier_Turn.h"
#include "GeomModifier_Weld.h"
#include "NavMesh/NavMeshBoundsVolume.h"
#include "NavigationSystem.h"
#include "GeometryModeModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogGeometryMode, Log, All);

/*------------------------------------------------------------------------------
	Geometry Editing.
------------------------------------------------------------------------------*/

FEdModeGeometry::FEdModeGeometry()
{
	Tools.Add( new FModeTool_GeometryModify() );
	SetCurrentTool( MT_GeometryModify );
}

FEdModeGeometry::~FEdModeGeometry()
{
	GeomObjects.Empty();
}

void FEdModeGeometry::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View,Viewport,PDI);

	RenderVertex( View, PDI );
	RenderEdge( View, PDI );
	RenderPoly( View, Viewport, PDI );
}

bool FEdModeGeometry::ShowModeWidgets() const
{
	return 1;
}

bool FEdModeGeometry::ShouldDrawBrushWireframe( AActor* InActor ) const
{
	checkSlow( InActor );

	// If the actor isn't selected, we don't want to interfere with it's rendering.
	if( !GEditor->GetSelectedActors()->IsSelected( InActor ) )
	{
		return true;
	}

	return true;//false;
}

bool FEdModeGeometry::GetCustomDrawingCoordinateSystem( FMatrix& InMatrix, void* InData )
{
	if( GetSelectionState() == GSS_None )
	{
		return 0;
	}

	if( InData )
	{
		FGeomBase* GeomBase = static_cast<FGeomBase*>(InData);
		FGeomObjectPtr GeomObject = GeomBase->GetParentObject();
		check(GeomObject.IsValid());
		ABrush* Brush = GeomObject->GetActualBrush();
		InMatrix = FRotationMatrix(GeomBase->GetNormal().Rotation()) * FQuatRotationMatrix(Brush->GetActorQuat());
	}
	else
	{
		// If we don't have a specific geometry object to get the normal from
		// use the one that was last selected.

		for( int32 o = 0 ; o < GeomObjects.Num() ; ++o )
		{
			FGeomObjectPtr go = GeomObjects[o];
			go->CompileSelectionOrder();

			if( go->SelectionOrder.Num() )
			{
				FGeomBase* GeomBase = go->SelectionOrder[go->SelectionOrder.Num() - 1];
				check(GeomBase != nullptr);
				FGeomObjectPtr GeomObject = GeomBase->GetParentObject();
				check(GeomObject.IsValid());
				ABrush* Brush = GeomObject->GetActualBrush();
				InMatrix = FRotationMatrix( go->SelectionOrder[ go->SelectionOrder.Num()-1 ]->GetWidgetRotation() ) * FQuatRotationMatrix(Brush->GetActorQuat());
				return 1;
			}
		}
	}

	return 0;
}

bool FEdModeGeometry::GetCustomInputCoordinateSystem( FMatrix& InMatrix, void* InData )
{
	return GetCustomDrawingCoordinateSystem( InMatrix, InData );
}

bool FEdModeGeometry::UsesToolkits() const
{
	return true;
}

void FEdModeGeometry::Enter()
{
	FEdMode::Enter();
	
	if (!Toolkit.IsValid())
	{
		Toolkit = MakeShareable(new FGeometryModeToolkit);
		Toolkit->Init(Owner->GetToolkitHost());
	}

	GetFromSource();
}

void FEdModeGeometry::Exit()
{
	if(GetWorld())
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		for (auto GeomObjectIt = GeomObjectItor(); GeomObjectIt; GeomObjectIt++)
		{
			ANavMeshBoundsVolume* Volume = Cast<ANavMeshBoundsVolume>((*GeomObjectIt)->GetActualBrush());
			if (Volume)
			{
				NavSys->OnNavigationBoundsUpdated(Volume);
			}
		}
	}

	FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
	Toolkit.Reset();

	FEdMode::Exit();

	GeomObjects.Empty();
}

void FEdModeGeometry::ActorSelectionChangeNotify()
{
	GetFromSource();
}

void FEdModeGeometry::MapChangeNotify()
{
	// If the map changes in some major way, just refresh all the geometry data.
	GetFromSource();
}

void FEdModeGeometry::AddReferencedObjects( FReferenceCollector& Collector )
{
	// Call parent implementation
	FEdMode::AddReferencedObjects( Collector );

	FModeTool_GeometryModify* mtgm = (FModeTool_GeometryModify*)FindTool( MT_GeometryModify );
	for (auto Itor = mtgm->ModifierIterator(); Itor; ++Itor)
	{
		Collector.AddReferencedObject( *Itor );
	}
}

/**
* Returns the number of objects that are selected.
*/

int32 FEdModeGeometry::CountObjectsSelected()
{
	return GeomObjects.Num();
}

/**
* Returns the number of polygons that are selected.
*/

int32 FEdModeGeometry::CountSelectedPolygons()
{
	int32 Count = 0;

	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 P = 0 ; P < GeomObject->PolyPool.Num() ; ++P )
		{
			if( GeomObject->PolyPool[P].IsSelected() )
			{
				Count++;
			}
		}
	}

	return Count;
}

/**
* Returns the polygons that are selected.
*
* @param	InPolygons	An array to fill with the selected polygons.
*/

void FEdModeGeometry::GetSelectedPolygons( TArray<FGeomPoly*>& InPolygons ) const
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 P = 0 ; P < GeomObject->PolyPool.Num() ; ++P )
		{
			if( GeomObject->PolyPool[P].IsSelected() )
			{
				InPolygons.Add( &GeomObject->PolyPool[P] );
			}
		}
	}
}

/**
* Returns true if the user has polygons selected.
*/

bool FEdModeGeometry::HavePolygonsSelected()
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 P = 0 ; P < GeomObject->PolyPool.Num() ; ++P )
		{
			if( GeomObject->PolyPool[P].IsSelected() )
			{
				return true;
			}
		}
	}

	return false;
}

/**
* Returns the number of edges that are selected.
*/

int32 FEdModeGeometry::CountSelectedEdges()
{
	int32 Count = 0;

	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 E = 0 ; E < GeomObject->EdgePool.Num() ; ++E )
		{
			if( GeomObject->EdgePool[E].IsSelected() )
			{
				Count++;
			}
		}
	}

	return Count;
}

/**
* Returns true if the user has edges selected.
*/

bool FEdModeGeometry::HaveEdgesSelected()
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 E = 0 ; E < GeomObject->EdgePool.Num() ; ++E )
		{
			if( GeomObject->EdgePool[E].IsSelected() )
			{
				return true;
			}
		}
	}

	return false;
}

/**
* Returns the edges that are selected.
*
* @param	InEdges	An array to fill with the selected edges.
*/
void FEdModeGeometry::GetSelectedEdges( TArray<FGeomEdge*>& InEdges ) const
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 E = 0 ; E < GeomObject->EdgePool.Num() ; ++E )
		{
			if( GeomObject->EdgePool[E].IsSelected() )
			{
				InEdges.Add( &GeomObject->EdgePool[E] );
			}
		}
	}
}

/**
* Returns the number of vertices that are selected.
*/

int32 FEdModeGeometry::CountSelectedVertices()
{
	int32 Count = 0;

	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 V = 0 ; V < GeomObject->VertexPool.Num() ; ++V )
		{
			if( GeomObject->VertexPool[V].IsSelected() )
			{
				Count++;
			}
		}
	}

	return Count;
}

/**
* Returns true if the user has vertices selected.
*/

bool FEdModeGeometry::HaveVerticesSelected()
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 V = 0 ; V < GeomObject->VertexPool.Num() ; ++V )
		{
			if( GeomObject->VertexPool[V].IsSelected() )
			{
				return true;
			}
		}
	}

	return false;
}

/**
* Fills an array with all selected vertices.
*
* @param	InVerts		An array to fill with the unique list of selected vertices.
*/
void FEdModeGeometry::GetSelectedVertices( TArray<FGeomVertex*>& InVerts ) const
{
	InVerts.Empty();

	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];

		for( int32 V = 0 ; V < GeomObject->VertexPool.Num() ; ++V )
		{
			if( GeomObject->VertexPool[V].IsSelected() )
			{
				InVerts.Add( &GeomObject->VertexPool[V] );
			}
		}
	}
}


/**
* Utility function that allow you to poll and see if certain sub elements are currently selected.
*
* Returns a combination of the flags in ESelectionStatus.
*/

int32 FEdModeGeometry::GetSelectionState()
{
	int32 Status = 0;

	if( HavePolygonsSelected() )
	{
		Status |= GSS_Polygon;
	}

	if( HaveEdgesSelected() )
	{
		Status |= GSS_Edge;
	}

	if( HaveVerticesSelected() )
	{
		Status |= GSS_Vertex;
	}

	return Status;
}

FVector FEdModeGeometry::GetWidgetLocation() const
{
	return FEdMode::GetWidgetLocation();
}

bool FEdModeGeometry::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return OtherModeID == FGeometryEditingModes::EM_Bsp;
}

bool FEdModeGeometry::ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const
{
	if (Actor->IsA<ABrush>())
	{
		TArray<FGeomVertex*> SelectedVertices;
		TArray<FGeomPoly*> SelectedPolys;
		TArray<FGeomEdge*> SelectedEdges;

		GetSelectedVertices(SelectedVertices);
		GetSelectedPolygons(SelectedPolys);
		GetSelectedEdges(SelectedEdges);

		if (SelectedVertices.Num() + SelectedPolys.Num() + SelectedEdges.Num() > 0)
		{
			InOutBox.Init();

			for (FGeomVertex* Vertex : SelectedVertices)
			{
				InOutBox += Vertex->GetWidgetLocation();
			}

			for (FGeomPoly* Poly : SelectedPolys)
			{
				InOutBox += Poly->GetWidgetLocation();
			}

			for (FGeomEdge* Edge : SelectedEdges)
			{
				InOutBox += Edge->GetWidgetLocation();
			}

			// Zoom out a little bit so you can see the selection
			InOutBox = InOutBox.ExpandBy(25);

			return true;
		}
	}

	return false;
}

// ------------------------------------------------------------------------------

/**
 * Deselects all edges, polygons, and vertices for all selected objects.
 */
void FEdModeGeometry::GeometrySelectNone(bool bStoreSelection, bool bResetPivot)
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];
		GeomObject->Select( 0 );

		for( int VertexIdx = 0 ; VertexIdx < GeomObject->EdgePool.Num() ; ++VertexIdx )
		{
			GeomObject->EdgePool[VertexIdx].Select( 0 );
		}
		for( int VertexIdx = 0 ; VertexIdx < GeomObject->PolyPool.Num() ; ++VertexIdx )
		{
			GeomObject->PolyPool[VertexIdx].Select( 0 );
		}
		for( int VertexIdx = 0 ; VertexIdx < GeomObject->VertexPool.Num() ; ++VertexIdx )
		{
			GeomObject->VertexPool[VertexIdx].Select( 0 );
		}

		GeomObject->SelectionOrder.Empty();
	}

	if (bStoreSelection)
	{
		FModeTool_GeometryModify* GeometryModifier = (FModeTool_GeometryModify*)FindTool(MT_GeometryModify);
		GeometryModifier->StoreAllCurrentGeomSelections();
	}

	if (bResetPivot && (GeomObjects.Num() > 0))
	{
		Owner->SetPivotLocation(GeomObjects[0]->GetActualBrush()->GetActorLocation(), false);
	}
}

void FEdModeGeometry::SelectionChanged( )
{
	StaticCastSharedPtr<FGeometryModeToolkit>(Toolkit)->OnGeometrySelectionChanged();
}

// ------------------------------------------------------------------------------

void FEdModeGeometry::RenderPoly( const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI )
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];
		
		FLinearColor UnselectedColor = GeomObject->GetActualBrush()->GetWireColor();
		UnselectedColor.A = .1f;

		FLinearColor SelectedColor = GetDefault<UEditorStyleSettings>()->SelectionColor;
		SelectedColor.A = .5f;

		// Allocate the material proxy and register it so it can be deleted properly once the rendering is done with it.
		FDynamicColoredMaterialRenderProxy* SelectedColorInstance = new FDynamicColoredMaterialRenderProxy(GEngine->GeomMaterial->GetRenderProxy(),SelectedColor );
		PDI->RegisterDynamicResource( SelectedColorInstance );

		FDynamicColoredMaterialRenderProxy* UnselectedColorInstance = new FDynamicColoredMaterialRenderProxy(GEngine->GeomMaterial->GetRenderProxy(),UnselectedColor);
		PDI->RegisterDynamicResource( UnselectedColorInstance );		

		// Render selected filled polygons.
		for( int32 PolyIdx = 0 ; PolyIdx < GeomObject->PolyPool.Num() ; ++PolyIdx )
		{
			const FGeomPoly* GeomPoly = &GeomObject->PolyPool[PolyIdx];
			PDI->SetHitProxy( new HGeomPolyProxy(GeomPoly->GetParentObject(),PolyIdx) );
			{
				FDynamicMeshBuilder MeshBuilder(View->GetFeatureLevel());

				TArray<FVector> Verts;

				// Look at the edge list and create a list of vertices to render from.

				FVector LastPos(0);

				for( int32 EdgeIdx = 0 ; EdgeIdx < GeomPoly->EdgeIndices.Num() ; ++EdgeIdx )
				{
					const FGeomEdge* GeomEdge = &GeomPoly->GetParentObject()->EdgePool[ GeomPoly->EdgeIndices[EdgeIdx] ];

					if( EdgeIdx == 0 )
					{
						Verts.Add( (FVector)GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[0] ] );
						LastPos = (FVector)GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[0] ];
					}
					else if( GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[0] ].Equals((FVector3f)LastPos ) )
					{
						Verts.Add( (FVector)GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[1] ] );
						LastPos = (FVector)GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[1] ];
					}
					else
					{
						Verts.Add( (FVector)GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[0] ] );
						LastPos = (FVector)GeomPoly->GetParentObject()->VertexPool[ GeomEdge->VertexIndices[0] ];
					}
				}

				// Draw Polygon Triangles
				const int32 VertexOffset = MeshBuilder.AddVertex((FVector3f)Verts[0], FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White);
				MeshBuilder.AddVertex((FVector3f)Verts[1], FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White);

				for( int32 VertIdx = 2 ; VertIdx < Verts.Num() ; ++VertIdx )
				{
					MeshBuilder.AddVertex((FVector3f)Verts[VertIdx], FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White);
					MeshBuilder.AddTriangle( VertexOffset + VertIdx - 1, VertexOffset, VertexOffset + VertIdx);
				}

				if( GeomPoly->IsSelected() )
				{
					MeshBuilder.Draw(PDI, GeomObject->GetActualBrush()->ActorToWorld().ToMatrixWithScale(), SelectedColorInstance, SDPG_World, false );
				}
				else
				{
					// We only draw unselected polygons in the perspective viewport
					if( !Viewport->GetClient()->IsOrtho() )
					{
						// Unselected polygons are drawn at the world level but are bumped slightly forward to avoid z-fighting
						MeshBuilder.Draw(PDI, GeomObject->GetActualBrush()->ActorToWorld().ToMatrixWithScale(), UnselectedColorInstance, SDPG_World, false );
					}
				}
			}
			PDI->SetHitProxy( NULL );
		}
	}
}

// ------------------------------------------------------------------------------

void FEdModeGeometry::RenderEdge( const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeometryObject = GeomObjects[ObjectIdx];
		const FColor WireColor = GeometryObject->GetActualBrush()->GetWireColor();

		// Edges
		for( int32 EdgeIdx = 0 ; EdgeIdx < GeometryObject->EdgePool.Num() ; ++EdgeIdx )
		{
			const FGeomEdge* GeometryEdge = &GeometryObject->EdgePool[EdgeIdx];
			const FColor Color = GeometryEdge->IsSelected() ? FColor(255,128,64) : WireColor;

			PDI->SetHitProxy( new HGeomEdgeProxy(GeometryObject,EdgeIdx) );
			{
				FVector V0 = (FVector)GeometryObject->VertexPool[ GeometryEdge->VertexIndices[0] ];
				FVector V1 = (FVector)GeometryObject->VertexPool[ GeometryEdge->VertexIndices[1] ];
				const FTransform ActorToWorld = GeometryObject->GetActualBrush()->ActorToWorld();

				V0 = ActorToWorld.TransformPosition( V0 );
				V1 = ActorToWorld.TransformPosition( V1 );

				PDI->DrawLine( V0, V1, Color, SDPG_Foreground );
			}
			PDI->SetHitProxy( NULL );
		}
	}
}

// ------------------------------------------------------------------------------

void FEdModeGeometry::RenderVertex( const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	for( int32 ObjectIdx = 0 ; ObjectIdx < GeomObjects.Num() ; ++ObjectIdx )
	{
		FGeomObjectPtr GeomObject = GeomObjects[ObjectIdx];
		check(GeomObject.IsValid());

		// Vertices

		FColor Color;
		float Scale;
		FVector Location;

		for( int32 VertIdx = 0 ; VertIdx < GeomObject->VertexPool.Num() ; ++VertIdx )
		{
			const FGeomVertex* GeomVertex = &GeomObject->VertexPool[VertIdx];
			check(GeomVertex);
			check(GeomObject->GetActualBrush());

			Location = (FVector)GeomObject->GetActualBrush()->ActorToWorld().TransformPosition( (FVector)*GeomVertex );
			Scale = static_cast<float>( View->WorldToScreen( Location ).W * ( 4.0 / View->UnscaledViewRect.Width() / View->ViewMatrices.GetViewToClip().M[0][0] ) );
			Color = GeomVertex->IsSelected() ? FColor(255,128,64) : GeomObject->GetActualBrush()->GetWireColor();

			PDI->SetHitProxy( new HGeomVertexProxy( GeomObject, VertIdx) );
			PDI->DrawSprite( Location, 4.f * Scale, 4.f * Scale, GEngine->DefaultBSPVertexTexture->GetResource(), Color, SDPG_Foreground, 0.0, 0.0, 0.0, 0.0 );
			PDI->SetHitProxy( NULL );
		}
	}
}

/**
 * Cache all the selected geometry on the object, and add to the array if any is found
 *
 * Return true if new object has been added to the array.
 */

bool FEdModeGeometry::CacheSelectedData( TArray<HGeomMidPoints>& raGeomData, const FGeomObject& rGeomObject ) const
{
	// Early out if this object doesn't have a brush
	if ( !rGeomObject.ActualBrush )
	{
		return false;
	}

	HGeomMidPoints GeomData;

	// Loop through all the verts caching their midpoint if they're selected
	for ( int32 i=0; i<rGeomObject.VertexPool.Num(); i++ )
	{
		const FGeomVertex& rGeomVertex = rGeomObject.VertexPool[i];
		if(rGeomVertex.IsSelected())
		{
			GeomData.VertexPool.Add( rGeomVertex.GetMidPoint() );
		}
	}

	// Loop through all the edges caching their midpoint if they're selected
	for ( int32 i=0; i<rGeomObject.EdgePool.Num(); i++ )
	{
		const FGeomEdge& rGeomEdge = rGeomObject.EdgePool[i];
		if(rGeomEdge.IsSelected())
		{
			GeomData.EdgePool.Add( rGeomEdge.GetMidPoint() );
		}
	}

	// Loop through all the polys caching their midpoint if they're selected
	for ( int32 i=0; i<rGeomObject.PolyPool.Num(); i++ )
	{
		const FGeomPoly& rGeomPoly = rGeomObject.PolyPool[i];
		if(rGeomPoly.IsSelected())
		{
			GeomData.PolyPool.Add( rGeomPoly.GetMidPoint() );
		}
	}
	
	// Only add the data to the array if there was anything that was selected
	bool bRet = ( ( GeomData.VertexPool.Num() + GeomData.EdgePool.Num() + GeomData.PolyPool.Num() ) > 0 ? true : false );
	if ( bRet )
	{
		// Make note of the brush this belongs to, then add
		GeomData.ActualBrush = rGeomObject.ActualBrush;
		raGeomData.Add( GeomData );
	}
	return bRet;
}

/**
 * Attempt to find all the new geometry using the cached data, and cache those new ones out
 *
 * Return true everything was found (or there was nothing to find)
 */
bool FEdModeGeometry::FindFromCache( TArray<HGeomMidPoints>& raGeomData, FGeomObject& rGeomObject, TArray<FGeomBase*>& raSelectedGeom ) const
{
	// Early out if this object doesn't have a brush
	if ( !rGeomObject.ActualBrush )
	{
		return true;
	}

	// Early out if we don't have anything cached
	if ( raGeomData.Num() == 0 )
	{
		return true;
	}

	// Loop through all the cached data, seeing if there's a match for the brush
	// Note: if GetMidPoint wasn't pure virtual this could be much nicer
	bool bRet = false;		// True if the brush that was parsed was found and all verts/edges/polys were located
	bool bFound = false;	// True if the brush that was parsed was found
	for( int32 i=0; i<raGeomData.Num(); i++ )
	{
		// Does this brush match the cached actor?
		HGeomMidPoints& rGeomData = raGeomData[i];
		if ( rGeomData.ActualBrush == rGeomObject.ActualBrush )
		{
			// Compare location of new midpoints with cached versions
			bFound = true;
			bool bSucess = true;		// True if all verts/edges/polys were located
			for ( int32 j=0; j<rGeomData.VertexPool.Num(); j++ )
			{
				const FVector& rGeomVector = rGeomData.VertexPool[j];
				for ( int32 k=0; k<rGeomObject.VertexPool.Num(); k++ )
				{
					// If we have a match select it and move on to the next one
					FGeomVertex& rGeomVertex = rGeomObject.VertexPool[k];
					if ( rGeomVector.Equals( rGeomVertex.GetMidPoint() ) )
					{
						// Add the new geometry to the to-be-selected pool, and remove from the data pool
						raSelectedGeom.Add(&rGeomVertex);
						rGeomData.VertexPool.RemoveAt(j--);
						break;
					}
				}
			}
			// If we didn't locate them all inform the user
			if ( rGeomData.VertexPool.Num() != 0 )
			{
				UE_LOGF(LogGeometryMode, Warning, "Unable to find %d Vertex(s) in new BSP", rGeomData.VertexPool.Num() );
				bSucess = false;
			}

			// Compare location of new midpoints with cached versions
			for ( int32 j=0; j<rGeomData.EdgePool.Num(); j++ )
			{
				const FVector& rGeomVector = rGeomData.EdgePool[j];
				for ( int32 k=0; k<rGeomObject.EdgePool.Num(); k++ )
				{
					// If we have a match select it and move on to the next one
					FGeomEdge& rGeomEdge = rGeomObject.EdgePool[k];
					if ( rGeomVector.Equals( rGeomEdge.GetMidPoint() ) )
					{
						// Add the new geometry to the to-be-selected pool, and remove from the data pool
						raSelectedGeom.Add(&rGeomEdge);
						rGeomData.EdgePool.RemoveAt(j--);
						break;
					}
				}
			}
			// If we didn't locate them all inform the user
			if ( rGeomData.EdgePool.Num() != 0 )
			{
				UE_LOGF(LogGeometryMode, Warning, "Unable to find %d Edge(s) in new BSP", rGeomData.EdgePool.Num() );
				bSucess = false;
			}

			// Compare location of new midpoints with cached versions
			for ( int32 j=0; j<rGeomData.PolyPool.Num(); j++ )
			{
				const FVector& rGeomVector = rGeomData.PolyPool[j];
				for ( int32 k=0; k<rGeomObject.PolyPool.Num(); k++ )
				{
					// If we have a match select it and move on to the next one
					FGeomPoly& rGeomPoly = rGeomObject.PolyPool[k];
					if ( rGeomVector.Equals( rGeomPoly.GetMidPoint() ) )
					{
						// Add the new geometry to the to-be-selected pool, and remove from the data pool
						raSelectedGeom.Add(&rGeomPoly);
						rGeomData.PolyPool.RemoveAt(j--);
						break;
					}
				}
			}
			// If we didn't locate them all inform the user
			if ( rGeomData.PolyPool.Num() != 0 )
			{
				UE_LOGF(LogGeometryMode, Warning, "Unable to find %d Poly(s) in new BSP", rGeomData.PolyPool.Num() );
				bSucess = false;
			}

			// If we didn't locate them all inform the user, then remove from the data pool
			if ( !bSucess )
			{
				UE_LOGF(LogGeometryMode, Warning, "Unable to resolve %ls Brush in new BSP, see above", *rGeomData.ActualBrush->GetName() );
			}
			bRet = bSucess;
			raGeomData.RemoveAt(i--);
			break;
		}
	}
	// If we didn't locate the brush inform the user
	if ( !bFound )
	{
		UE_LOGF(LogGeometryMode, Warning, "Unable to find %ls Brush(s) in new BSP", *rGeomObject.ActualBrush->GetName() );
	}
	return bRet;
}

/**
 * Select all the verts/edges/polys that were found
 *
 * Return true if successful
 */
bool FEdModeGeometry::SelectCachedData( TArray<FGeomBase*>& raSelectedGeom ) const
{
	// Early out if we don't have anything cached
	if ( raSelectedGeom.Num() == 0 )
	{
		return false;
	}

	check(Owner->IsModeActive(FGeometryEditingModes::EM_Geometry));

	// Backup widget position, we want it to be in the same position as it was previously too
	FVector PivLoc = Owner->PivotLocation;
	FVector SnapLoc = Owner->SnappedLocation;

	// Loop through all the geometry that should be selected
	for( int32 i=0; i<raSelectedGeom.Num(); i++ )
	{
		// Does this brush match the cached actor?
		FGeomBase* pGeom = raSelectedGeom[i];
		if ( pGeom )
		{
			pGeom->Select();
		}
	}

	// Restore the widget position
	Owner->SetPivotLocation( PivLoc, false );
	Owner->SnappedLocation = SnapLoc;
	
	StaticCastSharedPtr<FGeometryModeToolkit>(Toolkit)->OnGeometrySelectionChanged();

	return true;
}

#define BSP_RESELECT	// Attempt to reselect any geometry that was selected prior to the BSP being rebuilt
#define BSP_RESELECT__ALL_OR_NOTHING	// If any geometry can't be located, then don't select anything

/**
 * Compiles geometry mode information from the selected brushes.
 */

void FEdModeGeometry::GetFromSource()
{
	GWarn->BeginSlowTask( NSLOCTEXT("EditorModes", "GeometryMode-BeginRebuildingBSPTask", "Rebuilding BSP"), false);

	TArray<HGeomMidPoints> GeomData;

	// Go through each brush and update its components before updating below
	for( int32 i=0; i<GeomObjects.Num(); i++ )
	{
		FGeomObjectPtr GeomObject = GeomObjects[i];
		if(GeomObject.IsValid() && GeomObject->ActualBrush)
		{
#ifdef BSP_RESELECT
			// Cache any information that'll help us reselect the object after it's reconstructed
			CacheSelectedData( GeomData, *GeomObject );
#endif // BSP_RESELECT
			GeomObject->ActualBrush->UnregisterAllComponents();
			if (GeomObject->ActualBrush->GetWorld())
			{
				GeomObject->ActualBrush->RegisterAllComponents();
			}
		}		
	}
	GeomObjects.Empty();

	TArray<FGeomBase*> SelectedGeom;

	// Notify the selected actors that they have been moved.
	bool bFound = true;
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		ABrush* BrushActor = Cast< ABrush >( Actor );
		if ( BrushActor )
		{
			if( BrushActor->Brush != NULL )
			{
				FGeomObjectPtr GeomObject			= MakeShareable( new FGeomObject() );
				GeomObject->SetParentObjectIndex( GeomObjects.Add( GeomObject ) );
				GeomObject->ActualBrush			= BrushActor;
				GeomObject->GetFromSource();
#ifdef BSP_RESELECT
				// Attempt to find all the previously selected geometry on this object if everything has gone OK so far
				if ( bFound && !FindFromCache( GeomData, *GeomObject, SelectedGeom ) )
				{
#ifdef BSP_RESELECT__ALL_OR_NOTHING
					// If it didn't succeed, don't attempt to reselect anything as the user will only end up moving part of their previous selection
					UE_LOGF(LogGeometryMode, Warning, "Unable to find all previously selected geometry data, resetting selection" );
					SelectedGeom.Empty();
					GeomData.Empty();
					bFound = false;
#endif // BSP_RESELECT__ALL_OR_NOTHING
				}
#endif // BSP_RESELECT
			}
		}
	}

#ifdef BSP_RESELECT
	// Reselect anything that came close to the cached midpoints
	SelectCachedData( SelectedGeom );
#endif // BSP_RESELECT

	GWarn->EndSlowTask();
}

/**
 * Changes the source brushes to match the current geometry data.
 */

void FEdModeGeometry::SendToSource()
{
	for( int32 o = 0 ; o < GeomObjects.Num() ; ++o )
	{
		FGeomObjectPtr go = GeomObjects[o];

		go->SendToSource();
	}
}

bool FEdModeGeometry::FinalizeSourceData()
{
	bool Result = 0;

	for( int32 o = 0 ; o < GeomObjects.Num() ; ++o )
	{
		FGeomObjectPtr go = GeomObjects[o];

		if( go->FinalizeSourceData() )
		{
			Result = 1;
		}
	}

	return Result;
}

void FEdModeGeometry::PostUndo()
{
	// Rebuild the geometry data from the current brush state

	GetFromSource();
	
	// Restore selection information.
	for( int32 o = 0 ; o < GeomObjects.Num() ; ++o )
	{
		int32 Idx = 0;
		FGeomObjectPtr go = GeomObjects[o];

		ABrush* Actor = go->GetActualBrush();

		// First, clear the current selection
		go->SelectNone();

		// Next, restore the cached selection
		go->UpdateFromSelectionArray( Actor->SavedSelections );
		int32 res = go->SetPivotFromSelectionArray( Actor->SavedSelections );
		
		//use the centre of the actor if we didnt find a suitable selection
		if( res == INDEX_NONE )
		{
			Owner->SetPivotLocation( Actor->GetActorLocation() , false );
		}
		
		go->ForceLastSelectionIndex( res );
	}
}

bool FEdModeGeometry::ExecDelete()
{
	check( Owner->IsModeActive(FGeometryEditingModes::EM_Geometry ) );

	// Find the delete modifier and execute it.

	FModeTool_GeometryModify* mtgm = (FModeTool_GeometryModify*)FindTool( MT_GeometryModify );

	for( FModeTool_GeometryModify::TModifierIterator Itor( mtgm->ModifierIterator() ) ; Itor ; ++Itor )
	{
		UGeomModifier* gm = *Itor;

		if( gm->IsA( UGeomModifier_Delete::StaticClass()) )
		{
			if( gm->Apply() )
			{
				return 1;
			}
		}
	}

	return 0;
}

void FEdModeGeometry::UpdateInternalData()
{
	GetFromSource();
}



/*-----------------------------------------------------------------------------
	FModeTool_GeometryModify.
-----------------------------------------------------------------------------*/

void FModeTool_GeometryModify::SetCurrentModifier( UGeomModifier* InModifier )
{
	if( CurrentModifier )
	{
		CurrentModifier->WasDeactivated();
	}
	CurrentModifier = InModifier;
	CurrentModifier->WasActivated();
}

UGeomModifier* FModeTool_GeometryModify::GetCurrentModifier() const { return CurrentModifier; }

int32 FModeTool_GeometryModify::GetNumModifiers() const { return Modifiers.Num(); }

FModeTool_GeometryModify::FModeTool_GeometryModify()
{
	ID = MT_GeometryModify;

	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Edit::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Extrude::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Clip::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Pen::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Lathe::StaticClass() ) );

	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Create::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Delete::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Flip::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Split::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Triangulate::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Optimize::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Turn::StaticClass() ) );
	Modifiers.Add( NewObject<UGeomModifier>( GetTransientPackage(), UGeomModifier_Weld::StaticClass() ) );

	CurrentModifier = NULL;
	
	bGeomModified = false;
}

void FModeTool_GeometryModify::SelectNone()
{
	FEdModeGeometry* Mode = GLevelEditorModeTools().GetActiveModeTyped<FEdModeGeometry>(FGeometryEditingModes::EM_Geometry);
	check(Mode);
	Mode->GeometrySelectNone(false, false);
}

/** @return		true if something was selected/deselected, false otherwise. */
bool FModeTool_GeometryModify::BoxSelect( FBox& InBox, bool InSelect )
{
	bool bResult = false;
	if( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry ) )
	{
		FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry );

		for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
		{
			FGeomObjectPtr go = *Itor;
			FTransform ActorToWorld = go->GetActualBrush()->ActorToWorld();

			// Only verts for box selection

			for( int32 v = 0 ; v < go->VertexPool.Num() ; ++v )
			{
				FGeomVertex& gv = go->VertexPool[v];
				if( FMath::PointBoxIntersection( ActorToWorld.TransformPosition( gv.GetMid() ), InBox ) )
				{
					gv.Select( InSelect );
					bResult = true;
				}
			}
		}
	}
	return bResult;
}

/** @return		true if something was selected/deselected, false otherwise. */
bool FModeTool_GeometryModify::FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect /* = true */ )
{
	bool bResult = false;
	if( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry ) )
	{
		FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry );

		for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
		{
			FGeomObjectPtr go = *Itor;
			FTransform ActorToWorld = go->GetActualBrush()->ActorToWorld();
			// Check each vertex to see if its inside the frustum
			for( int32 v = 0 ; v < go->VertexPool.Num() ; ++v )
			{
				FGeomVertex& gv = go->VertexPool[v];
				if( InFrustum.IntersectBox( ActorToWorld.TransformPosition( gv.GetMid() ), FVector::ZeroVector ) )
				{
					gv.Select( InSelect );
					bResult = true;
				}
			}
		}
	}
	return bResult;
}

void FModeTool_GeometryModify::Tick(FEditorViewportClient* ViewportClient,float DeltaTime)
{
	if( CurrentModifier )
	{
		CurrentModifier->Tick( ViewportClient, DeltaTime );
	}
}

/**
 * @return		true if the delta was handled by this editor mode tool.
 */
bool FModeTool_GeometryModify::InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale)
{
	bool bResult = false;
	if( InViewportClient->GetCurrentWidgetAxis() != EAxisList::None )
	{
		// Geometry mode passes the input on to the current modifier.
		if( CurrentModifier )
		{
			bResult = CurrentModifier->InputDelta( InViewportClient, InViewport, InDrag, InRot, InScale );
		}
	}
	return bResult;
}

bool FModeTool_GeometryModify::StartModify()
{
	// Let the modifier do any set up work that it needs to.
	if( CurrentModifier != NULL )
	{
		//Store the current state of the brush so that we can return to upon faulty operation
		CurrentModifier->CacheBrushState();

		return CurrentModifier->StartModify();
	}

	// Clear modified flag, so we can track if something actually changes before EndModify
	bGeomModified = false;

	// No modifier to start
	return false;
}

bool FModeTool_GeometryModify::EndModify()
{
	// Let the modifier finish up.
	if( CurrentModifier != NULL )
	{
		FEdModeGeometry* mode = ((FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry));

		// Update the source data to match the current geometry data.
		mode->SendToSource();

		// Make sure the source data has remained viable.
		if( mode->FinalizeSourceData() )
		{
			// If the source data was modified, reconstruct the geometry data to reflect that.
			mode->GetFromSource();
		}

		CurrentModifier->EndModify();

		// Update internals.
		for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
		{
			FGeomObjectPtr go = *Itor;
			go->ComputeData();
			FBSPOps::bspUnlinkPolys( go->GetActualBrush()->Brush );			
			
			// If geometry was actually changed, call PostEditBrush
			if(bGeomModified)
			{
				ABrush* Brush = go->GetActualBrush();
				if(Brush)
				{
					if(!Brush->IsStaticBrush())
					{
						FBSPOps::csgPrepMovingBrush(Brush);
					}
				}
				bGeomModified = false;
			}
		}		
	}

	return 1;
}

void FModeTool_GeometryModify::StartTrans()
{
	if( CurrentModifier != NULL )
	{
		CurrentModifier->StartTrans();
	}
}

void FModeTool_GeometryModify::EndTrans()
{
	if( CurrentModifier != NULL )
	{
		CurrentModifier->EndTrans();
	}
}

/**
 * @return		true if the key was handled by this editor mode tool.
 */
bool FModeTool_GeometryModify::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	check( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry ) );
	// Give the current modifier a chance to handle this first
	if( CurrentModifier && CurrentModifier->InputKey( ViewportClient, Viewport, Key, Event ) )
	{
		return true;
	}

	if (Key == EKeys::Escape)
	{
		// Hitting ESC will deselect any subobjects first.  If no subobjects are selected, then it will
		// deselect the brushes themselves.

		FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry );
		bool bHadSubObjectSelections = (mode->GetSelectionState() > 0) ? true : false;

		for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
		{
			FGeomObjectPtr go = *Itor;

			for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
			{
				FGeomPoly& gp = go->PolyPool[p];
				if( gp.IsSelected() )
				{
					gp.Select( false );
					bHadSubObjectSelections = true;
				}
			}

			for( int32 e = 0 ; e < go->EdgePool.Num() ; ++e )
			{
				FGeomEdge& ge = go->EdgePool[e];
				if( ge.IsSelected() )
				{
					ge.Select( false );
					bHadSubObjectSelections = true;
				}
			}

			for( int32 v = 0 ; v < go->VertexPool.Num() ; ++v )
			{
				FGeomVertex& gv = go->VertexPool[v];
				if( gv.IsSelected() )
				{
					gv.Select( false );
					bHadSubObjectSelections = true;
				}
			}
		}

		if( bHadSubObjectSelections )
		{
			GEditor->RedrawAllViewports();
			return true;
		}
	}
	else
	{
		return FModeTool::InputKey( ViewportClient, Viewport, Key, Event );
	}

	return false;
}

void FModeTool_GeometryModify::DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas)
{
	// Give the current modifier a chance to draw a HUD

	if( CurrentModifier )
	{
		CurrentModifier->DrawHUD( ViewportClient, Viewport, View, Canvas );
	}
}

void FModeTool_GeometryModify::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
	// Give the current modifier a chance to render

	if( CurrentModifier )
	{
		CurrentModifier->Render( View, Viewport, PDI );
	}
}

void FModeTool_GeometryModify::StoreAllCurrentGeomSelections()
{
	if (CurrentModifier)
	{
		CurrentModifier->StoreAllCurrentGeomSelections();
	}
}
