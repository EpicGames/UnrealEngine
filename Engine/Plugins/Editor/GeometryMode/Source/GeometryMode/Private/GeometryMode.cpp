// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMode.h"
#include "DetailsViewArgs.h"
#include "Widgets/Text/STextBlock.h"
#include "Misc/NotifyHook.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorModeManager.h"
#include "GeometryEdMode.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "GeomModifier.h"
#include "GeometryModeModule.h"
#include "GeometryModeStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "GeometryMode"


namespace GeometryModePaletteNames
{
	static const FName ToolsPalette(TEXT("Tools"));
};

/** Geometry Mode widget for controls */
class SGeometryModeControls : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SGeometryModeControls) {}
	SLATE_END_ARGS()

public:

	void SelectionChanged();

	/** SCompoundWidget functions */
	void Construct(const FArguments& InArgs, TSharedRef<FGeometryModeToolkit> GeometryMode);

	void SetPropertyObjects(const TArray<UObject*>& PropertyObjects);

protected:
	/** Returns the visibility state of the properties control */
	bool ArePropertiesVisible() const;

	EVisibility GetPropertyVisibility() const;

	/** Called when the Apply button is clicked */
	FReply OnApplyClicked();

	/** Called when a modifier button is clicked */
	FReply OnModifierClicked(UGeomModifier* Modifier);

private:
	/** Creates the geometry mode controls */
	void CreateLayout(TSharedRef<FGeometryModeToolkit>& GeometryMode);

	/** Creates controls for the modifiers section */
	TSharedRef<SVerticalBox> CreateTopModifierButtons(TSharedRef<FGeometryModeToolkit>& GeometryMode);

	/** Creates controls for the modifier properties section */
	TSharedRef<class IDetailsView> CreateModifierProperties(TSharedRef<FGeometryModeToolkit>& GeometryMode);

	/** Creates a modifier radio button */
	TSharedRef<SCheckBox> CreateSingleModifierRadioButton(UGeomModifier* Modifier, TSharedRef<FGeometryModeToolkit>& GeometryMode);

	/** Creates an action button */
	TSharedRef<SButton> CreateSingleModifierButton(UGeomModifier* Modifier, TSharedRef<FGeometryModeToolkit>& GeometryMode);

	void MakeBuilderBrush(UClass* BrushBuilderClass);

	void OnAddVolume(UClass* VolumeClass);

private:
	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindow;

	/** A list of the checkbox modifier controls */
	TArray< TSharedPtr<SCheckBox> > ModifierControls;

	TWeakPtr<FGeometryModeToolkit> GeometryModeWeakPtr;

	/** The properties control */
	TSharedPtr<class IDetailsView> PropertiesControl;
};

void SGeometryModeControls::SelectionChanged()
{
	if (const TSharedPtr<FGeometryModeToolkit> GeometryModeSharedPtr = GeometryModeWeakPtr.Pin())
	{
		if (FModeTool_GeometryModify* ModeTool = GeometryModeSharedPtr->GetGeometryModeTool())
		{
			if (!ModeTool->GetCurrentModifier()->SupportsCurrentSelection() && ModeTool->GetNumModifiers() > 0)
			{
				ModeTool->SetCurrentModifier(ModeTool->GetModifier(0));
			}
		}
	}
}

void SGeometryModeControls::Construct(const FArguments& InArgs, TSharedRef<FGeometryModeToolkit> GeometryMode)
{
	GeometryModeWeakPtr = GeometryMode;

	if (FModeTool_GeometryModify* ModeTool = GeometryMode->GetGeometryModeTool())
	{
		if (ModeTool->GetNumModifiers() > 0)
		{
			ModeTool->SetCurrentModifier(ModeTool->GetModifier(0));
		}

		CreateLayout(GeometryMode);
	}
}

void SGeometryModeControls::SetPropertyObjects(const TArray<UObject*>& PropertyObjects)
{
	PropertiesControl->SetObjects(PropertyObjects);
}

bool SGeometryModeControls::ArePropertiesVisible() const
{
	if (const TSharedPtr<FGeometryModeToolkit> GeometryModeSharedPtr = GeometryModeWeakPtr.Pin())
	{
		if (FModeTool_GeometryModify* ModeTool = GeometryModeSharedPtr->GetGeometryModeTool())
		{
			return (ModeTool->GetNumModifiers() > 0) && (ModeTool->GetCurrentModifier() != ModeTool->GetModifier(0));
		}
	}
	return false;
}

EVisibility SGeometryModeControls::GetPropertyVisibility() const
{
	if(ArePropertiesVisible())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

void SGeometryModeControls::CreateLayout(TSharedRef<FGeometryModeToolkit>& GeometryMode)
{
	this->ChildSlot
	[
		SNew(SScrollBox)
		+SScrollBox::Slot()
		.Padding(0.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				CreateModifierProperties(GeometryMode)
			]	
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
				.Visibility(this, &SGeometryModeControls::GetPropertyVisibility)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("SGeometryModeDialog_Apply", "Apply"))
				.Visibility(this, &SGeometryModeControls::GetPropertyVisibility)
				.OnClicked(GeometryMode, &FGeometryModeToolkit::OnApplyClicked)
			]
		]
	];
}

TSharedRef<SVerticalBox> SGeometryModeControls::CreateTopModifierButtons(TSharedRef<FGeometryModeToolkit>& GeometryMode)
{
	FModeTool_GeometryModify* GeometryModeTool = GeometryMode->GetGeometryModeTool();
	TSharedPtr<SVerticalBox> Vbox;
	const TSharedRef<SGridPanel> RadioButtonPanel = SNew(SGridPanel);

	// Loop through all geometry modifiers and create radio buttons for ones with the bPushButton set to false
	int32 CurrentModifierButtonCount = 0;

	// Add the Apply button
	SAssignNew(Vbox, SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		RadioButtonPanel
	];

	return Vbox.ToSharedRef();
}

TSharedRef<IDetailsView> SGeometryModeControls::CreateModifierProperties(TSharedRef<FGeometryModeToolkit>& GeometryMode)
{
	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.bAllowSearch = false;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertiesControl = PropertyModule.CreateDetailView(Args);
	PropertiesControl->SetVisibility(TAttribute<EVisibility>(this, &SGeometryModeControls::GetPropertyVisibility));

	return PropertiesControl.ToSharedRef();
}

TSharedRef<SCheckBox> SGeometryModeControls::CreateSingleModifierRadioButton(UGeomModifier* Modifier, TSharedRef<FGeometryModeToolkit>& GeometryMode)
{
	TSharedRef<SCheckBox> CheckBox =
	SNew(SCheckBox)
	.Style(FAppStyle::Get(), "RadioButton")
	.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
	.IsChecked(GeometryMode, &FGeometryModeToolkit::IsModifierChecked, Modifier)
	.IsEnabled(GeometryMode, &FGeometryModeToolkit::IsModifierEnabled, Modifier)
	.OnCheckStateChanged(GeometryMode, &FGeometryModeToolkit::OnModifierStateChanged, Modifier)
	.ToolTip(SNew(SToolTip).Text(Modifier->GetModifierTooltip()))
	[
		SNew(STextBlock).Text( Modifier->GetModifierDescription() )
	];

	ModifierControls.Add(CheckBox);

	return CheckBox;
}

TSharedRef<SButton> SGeometryModeControls::CreateSingleModifierButton(UGeomModifier* Modifier, TSharedRef<FGeometryModeToolkit>& GeometryMode)
{
	TSharedRef<SButton> Widget =
	SNew(SButton)
	.Text( Modifier->GetModifierDescription() )
	.ToolTip(SNew(SToolTip).Text(Modifier->GetModifierTooltip()))
	.HAlign(HAlign_Center)
	.IsEnabled(GeometryMode, &FGeometryModeToolkit::IsModifierEnabled, Modifier)
	.OnClicked_Lambda(
		[this, Modifier]()->FReply
		{
			GeometryModeWeakPtr.Pin()->OnModifierClicked(Modifier);
			return FReply::Handled();
		}
	);

	return Widget;
}

void FGeometryModeToolkit::Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost)
{
	GeomWidget = SNew(SGeometryModeControls, SharedThis(this));
	FModeToolkit::Init(InitToolkitHost);
}

FName FGeometryModeToolkit::GetToolkitFName() const
{
	return FName("GeometryMode");
}

FText FGeometryModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT( "ToolkitName", "Geometry Mode" );
}

class FEdMode* FGeometryModeToolkit::GetEditorMode() const
{
	return (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
}

void FGeometryModeToolkit::OnGeometrySelectionChanged()
{
	GeomWidget->SelectionChanged();
}

class FModeTool_GeometryModify* FGeometryModeToolkit::GetGeometryModeTool() const
{
	if (FEdModeGeometry* Mode = static_cast<FEdModeGeometry*>(GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry)))
	{
		return static_cast<FModeTool_GeometryModify*>(Mode->GetCurrentTool());
	}

	return nullptr;
}

void FGeometryModeToolkit::OnModifierStateChanged(ECheckBoxState NewCheckedState, UGeomModifier* Modifier)
{
	if (NewCheckedState == ECheckBoxState::Checked)
	{
		if (FModeTool_GeometryModify* ModeTool = GetGeometryModeTool())
		{
			ModeTool->SetCurrentModifier(Modifier);

			TArray<UObject*> PropertyObjects;
			PropertyObjects.Add(ModeTool->GetCurrentModifier());

			GeomWidget->SetPropertyObjects(PropertyObjects);
		}
	}
}

void FGeometryModeToolkit::OnModifierToolBarButtonClicked(UGeomModifier* Modifier)
{
	if (const FModeTool_GeometryModify* ModeTool = GetGeometryModeTool())
	{
		const ECheckBoxState NewState = ModeTool->GetCurrentModifier() == Modifier ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;

		OnModifierStateChanged(NewState, Modifier);
	}
}

ECheckBoxState FGeometryModeToolkit::IsModifierChecked(UGeomModifier* Modifier) const
{
	if (const FEdModeGeometry* Mode = (const FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry))
	{
		if (const FModeTool_GeometryModify* Tool = (const FModeTool_GeometryModify*) Mode->GetCurrentTool())
		{
			return (Tool->GetCurrentModifier() == Modifier)
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Unchecked;
}

FReply FGeometryModeToolkit::OnApplyClicked()
{
	check(GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry));

	if (const FModeTool_GeometryModify* ModeTool = GetGeometryModeTool())
	{
		ModeTool->GetCurrentModifier()->Apply();

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void FGeometryModeToolkit::OnModifierClicked(UGeomModifier* Modifier)
{
	check(GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry));

	Modifier->Apply();
}

bool FGeometryModeToolkit::IsModifierEnabled(UGeomModifier* Modifier) const
{
	return Modifier->SupportsCurrentSelection();
}


TSharedPtr<SWidget> FGeometryModeToolkit::GetInlineContent() const
{
	return GeomWidget;
}

void FGeometryModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames.Add(GeometryModePaletteNames::ToolsPalette);
}

FText FGeometryModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	if (Palette == GeometryModePaletteNames::ToolsPalette)
	{
		return LOCTEXT("GeometryMode_ToolsPalette", "Tools");
	}

	return FText::GetEmpty();
}

void FGeometryModeToolkit::BuildToolPalette(FName Palette, class FToolBarBuilder& ToolbarBuilder)
{
	if (Palette == GeometryModePaletteNames::ToolsPalette)
	{
		FModeTool_GeometryModify* GeometryModeTool = GetGeometryModeTool();
		if (!GeometryModeTool)
		{
			return;
		}

		// Loop through all geometry modifiers and create radio buttons for ones with the bPushButton set to false
		//int32 CurrentModifierButtonCount = 0;
		for (FModeTool_GeometryModify::TModifierIterator Itor(GeometryModeTool->ModifierIterator()); Itor; ++Itor)
		{
			UGeomModifier* Modifier = *Itor;
			if (!Modifier->bPushButton && Modifier->bAppearsInToolbar)
			{
				FUIAction ModifierAction;

				ModifierAction.ExecuteAction = FExecuteAction::CreateSP(this, &FGeometryModeToolkit::OnModifierToolBarButtonClicked, Modifier);
				ModifierAction.GetActionCheckState = FGetActionCheckState::CreateSP(this, &FGeometryModeToolkit::IsModifierChecked, Modifier);
				ModifierAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &FGeometryModeToolkit::IsModifierEnabled, Modifier);

				FSlateIcon Icon(FGeometryModeStyle::GetStyleSetName(), Modifier->ToolbarIconName);

				ToolbarBuilder.AddToolBarButton(ModifierAction, NAME_None, Modifier->GetModifierDescription(), Modifier->GetModifierTooltip(), Icon, EUserInterfaceActionType::RadioButton);
			}
		}

		ToolbarBuilder.AddSeparator();

		// Loop through all geometry modifiers and create buttons for ones with the bPushButton set to true
		int32 PushButtonId = 0;
		for (FModeTool_GeometryModify::TModifierIterator Itor(GeometryModeTool->ModifierIterator()); Itor; ++Itor)
		{
			UGeomModifier* Modifier = *Itor;
			if (Modifier->bPushButton && Modifier->bAppearsInToolbar)
			{
				FUIAction ModifierAction;

				ModifierAction.ExecuteAction = FExecuteAction::CreateSP(this, &FGeometryModeToolkit::OnModifierClicked, Modifier);
				ModifierAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &FGeometryModeToolkit::IsModifierEnabled, Modifier);

				FSlateIcon Icon(FGeometryModeStyle::GetStyleSetName(), Modifier->ToolbarIconName);

				ToolbarBuilder.AddToolBarButton(ModifierAction, NAME_None, Modifier->GetModifierDescription(), Modifier->GetModifierTooltip(), Icon, EUserInterfaceActionType::Button);
			}
		}
	}
}

FText FGeometryModeToolkit::GetActiveToolDisplayName() const
{
	if (const FModeTool_GeometryModify* ModeTool = GetGeometryModeTool())
	{
		return ModeTool->GetCurrentModifier()->GetModifierDescription();//  + LOCTEXT("GeometryMode_ToolsTool", " Tool");
	}

	return FText();
}

FText FGeometryModeToolkit::GetActiveToolMessage() const
{
	if (const FModeTool_GeometryModify* ModeTool = GetGeometryModeTool())
	{
		UGeomModifier* CurrentModifier = ModeTool->GetCurrentModifier();
		return ( CurrentModifier && CurrentModifier->SupportsCurrentSelection() )
			? CurrentModifier->GetModifierTooltip()
			: LOCTEXT("GeometryMode_NoBrushSelectedMessage", "Select a brush actor to begin editing its geometry");
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
