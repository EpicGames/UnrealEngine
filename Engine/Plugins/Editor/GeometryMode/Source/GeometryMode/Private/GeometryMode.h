// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"

enum class ECheckBoxState : uint8;

class IDetailsView;
class SButton;
class SCheckBox;
class SUniformGridPanel;
class UGeomModifier;
class FGeometryModeToolkit;




/**
 * Mode Toolkit for the Geometry Tools
 */
class FGeometryModeToolkit : public FModeToolkit
{
	friend class SGeometryModeControls;

public:

	/** Initializes the geometry mode toolkit */
	virtual void Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost) override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual class FEdMode* GetEditorMode() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override;

	virtual void GetToolPaletteNames(TArray<FName>& PaletteNames) const override;
	virtual FText GetToolPaletteDisplayName(FName Palette) const override;
	virtual void BuildToolPalette(FName Palette, class FToolBarBuilder& ToolbarBuilder) override;

	/** Modes Panel Header Information **/
	virtual FText GetActiveToolDisplayName() const;
	virtual FText GetActiveToolMessage() const;

	/** Method called when the selection */
	void OnGeometrySelectionChanged();

	/** Returns a reference to the geometry mode tool */
	class FModeTool_GeometryModify* GetGeometryModeTool() const;

private:
	/** Called when a new modifier mode is selected */
	void OnModifierStateChanged(ECheckBoxState NewCheckedState, UGeomModifier* Modifier);

	void OnModifierToolBarButtonClicked(UGeomModifier* Modifier);

	/** Returns the state of a modifier radio button */
	ECheckBoxState IsModifierChecked(UGeomModifier* Modifier) const;

	/** Returns the enabled state of a modifier button */
	bool IsModifierEnabled(UGeomModifier* Modifier) const;

	FReply OnApplyClicked();

	void OnModifierClicked(UGeomModifier* Modifier);

private:
	/** Geometry tools widget */
	TSharedPtr<class SGeometryModeControls> GeomWidget;
};
