// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryModeModule.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "GeometryEdMode.h"
#include "GeometryModeStyle.h"
#include "Styling/AppStyle.h"


FEditorModeID FGeometryEditingModes::EM_Geometry = FEditorModeID(TEXT("EM_Geometry"));
FEditorModeID FGeometryEditingModes::EM_Bsp = FEditorModeID(TEXT("EM_Bsp"));
FEditorModeID FGeometryEditingModes::EM_TextureAlign = FEditorModeID(TEXT("EM_TextureAlign"));

void FGeometryModeModule::StartupModule()
{
	FGeometryModeStyle::Initialize();

	FEditorModeRegistry::Get().RegisterMode<FEdModeGeometry>(
		FGeometryEditingModes::EM_Geometry,
		NSLOCTEXT("EditorModes", "GeometryMode", "Brush Editing"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.BrushEdit"),
		true,
		7000
	);
}


void FGeometryModeModule::ShutdownModule()
{
	FEditorModeRegistry::Get().UnregisterMode(FGeometryEditingModes::EM_Geometry);

	FGeometryModeStyle::Shutdown();
}

IMPLEMENT_MODULE(FGeometryModeModule, GeometryMode);
