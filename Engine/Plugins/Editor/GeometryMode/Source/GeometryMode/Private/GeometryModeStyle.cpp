// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryModeStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FGeometryModeStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

FString FGeometryModeStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("GeometryMode"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FGeometryModeStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FGeometryModeStyle::Get() { return StyleSet; }

FName FGeometryModeStyle::GetStyleSetName()
{
	static FName GeometryModeStyleName(TEXT("GeometryModeStyle"));
	return GeometryModeStyleName;
}

void FGeometryModeStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon20x20(20.0f, 20.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	StyleSet->Set("GeometryMode.EditTool",              new IMAGE_PLUGIN_BRUSH("Icons/BSP_Edit_20x", Icon20x20));
	StyleSet->Set("GeometryMode.EditTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/BSP_Edit_20x", Icon20x20));
	StyleSet->Set("GeometryMode.ExtrudeTool",           new IMAGE_PLUGIN_BRUSH("Icons/BSP_Extrude_20x", Icon20x20));
	StyleSet->Set("GeometryMode.ExtrudeTool.Small",     new IMAGE_PLUGIN_BRUSH("Icons/BSP_Extrude_20x", Icon20x20));
	StyleSet->Set("GeometryMode.LatheTool",             new IMAGE_PLUGIN_BRUSH("Icons/BSP_Lathe_20x", Icon20x20));
	StyleSet->Set("GeometryMode.LatheTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/BSP_Lathe_20x", Icon20x20));
	StyleSet->Set("GeometryMode.PenTool",               new IMAGE_PLUGIN_BRUSH("Icons/BSP_Pen_20x", Icon20x20));
	StyleSet->Set("GeometryMode.PenTool.Small",         new IMAGE_PLUGIN_BRUSH("Icons/BSP_Pen_20x", Icon20x20));
	StyleSet->Set("GeometryMode.ClipTool",              new IMAGE_PLUGIN_BRUSH("Icons/BSP_Clip_20x", Icon20x20));
	StyleSet->Set("GeometryMode.ClipTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/BSP_Clip_20x", Icon20x20));
	StyleSet->Set("GeometryMode.FlipTool",              new IMAGE_PLUGIN_BRUSH("Icons/BSP_Flip_20x", Icon20x20));
	StyleSet->Set("GeometryMode.FlipTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/BSP_Flip_20x", Icon20x20));
	StyleSet->Set("GeometryMode.SplitTool",             new IMAGE_PLUGIN_BRUSH("Icons/BSP_Split_20x", Icon20x20));
	StyleSet->Set("GeometryMode.SplitTool.Small",       new IMAGE_PLUGIN_BRUSH("Icons/BSP_Split_20x", Icon20x20));
	StyleSet->Set("GeometryMode.TriangulateTool",       new IMAGE_PLUGIN_BRUSH("Icons/BSP_Triangulate_20x", Icon20x20));
	StyleSet->Set("GeometryMode.TriangulateTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/BSP_Triangulate_20x", Icon20x20));
	StyleSet->Set("GeometryMode.OptimizeTool",          new IMAGE_PLUGIN_BRUSH("Icons/BSP_Optimize_20x", Icon20x20));
	StyleSet->Set("GeometryMode.OptimizeTool.Small",    new IMAGE_PLUGIN_BRUSH("Icons/BSP_Optimize_20x", Icon20x20));
	StyleSet->Set("GeometryMode.TurnTool",              new IMAGE_PLUGIN_BRUSH("Icons/BSP_Turn_20x", Icon20x20));
	StyleSet->Set("GeometryMode.TurnTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/BSP_Turn_20x", Icon20x20));
	StyleSet->Set("GeometryMode.weldTool",              new IMAGE_PLUGIN_BRUSH("Icons/BSP_Weld_20x", Icon20x20));
	StyleSet->Set("GeometryMode.weldTool.Small",        new IMAGE_PLUGIN_BRUSH("Icons/BSP_Weld_20x", Icon20x20));
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FGeometryModeStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
