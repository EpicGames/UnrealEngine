// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MessageDialog.h"
#include "SceneView.h"
#include "Model.h"
#include "Selection.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/Polys.h"
#include "EditorModeManager.h"
#include "Dialogs/Dialogs.h"
#include "BSPOps.h"
#include "Engine/BrushBuilder.h"
#include "GeometryEdMode.h"
#include "EditorGeometry.h"
#include "Engine/BrushShape.h"
#include "EditorSupportDelegates.h"
#include "ScopedTransaction.h"
#include "LevelEditorViewport.h"
#include "Layers/LayersSubsystem.h"
#include "ActorEditorUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Components/BrushComponent.h"
#include "GeomModifier_Clip.h"
#include "GeomModifier_Create.h"
#include "GeomModifier_Delete.h"
#include "GeomModifier_Extrude.h"
#include "GeomModifier_Flip.h"
#include "GeomModifier_Lathe.h"
#include "GeomModifier_Pen.h"
#include "GeomModifier_Split.h"
#include "GeomModifier_Optimize.h"
#include "GeomModifier_Turn.h"
#include "GeomModifier_Weld.h"
#include "GeometryModeModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogGeomModifier, Log, All);

#define LOCTEXT_NAMESPACE "UnrealEd.GeomModifier"

static FVector ComputeWorldSpaceMousePos( FEditorViewportClient* ViewportClient )
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport,
		ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags)
		.SetRealtimeUpdate( ViewportClient->IsRealtime() ));
	FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);

	// Note only works for ortho viewports
	const float X = static_cast<float>(ViewportClient->Viewport->GetMouseX());
	const float Y = static_cast<float>(ViewportClient->Viewport->GetMouseY());
	constexpr float Z = 0.5f; 
	return View->PixelToWorld(X, Y, Z);
}

UGeomModifier::UGeomModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bPushButton = false;
	bInitialized = false;
	bPendingPivotOffsetUpdate = false;
	CachedPolys = NULL;
	bAppearsInToolbar = true; 
}


const FText& UGeomModifier::GetModifierDescription() const
{ 
	return Description;
}

const FText& UGeomModifier::GetModifierTooltip() const
{
	return Tooltip;
}

void UGeomModifier::Initialize()
{
}

bool UGeomModifier::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	return false;
}

bool UGeomModifier::InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale)
{
	if( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		if( !bInitialized )
		{
			Initialize();
			bInitialized = true;
		}
	}

	return false;
}


bool UGeomModifier::Apply()
{
	bool bResult = false;
	if( GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		StartTrans();
		bResult = OnApply();
		EndTrans();
		EndModify();
	}
	return bResult;
}


bool UGeomModifier::OnApply()
{
	return false;
}


void UGeomModifier::GeomError(const FString& InErrorMsg)
{
	FMessageDialog::Open( EAppMsgType::Ok, FText::Format( NSLOCTEXT("UnrealEd", "Error_Modifier", "Modifier ({0}) : {1}"), GetModifierDescription(), FText::FromString(InErrorMsg) ) );
}


bool UGeomModifier::StartModify()
{
	bInitialized = false;
	bPendingPivotOffsetUpdate = false;
	return false;
}


bool UGeomModifier::EndModify()
{
	if (bPendingPivotOffsetUpdate)
	{
		UpdatePivotOffset();
	}
	StoreAllCurrentGeomSelections();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	return true;
}

void UGeomModifier::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
}

void UGeomModifier::DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas)
{
}


void UGeomModifier::CacheBrushState()
{
	FEdModeGeometry* GeomMode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	ABrush* BuilderBrush = GeomMode->GetWorld()->GetDefaultBrush();
	if( !CachedPolys )
	{
		//Create the list of polys
		CachedPolys = NewObject<UPolys>(this);
	}
	CachedPolys->Element.Empty();

	//Create duplicates of all of the polys in the brush
	for( int32 polyIndex = 0 ; polyIndex < BuilderBrush->Brush->Polys->Element.Num() ; ++polyIndex )
	{
		FPoly currentPoly = BuilderBrush->Brush->Polys->Element[polyIndex];
		FPoly newPoly;
		newPoly.Init();
		newPoly.Base = currentPoly.Base;

		//Add all of the verts to the new poly
		for( int32 vertIndex = 0; vertIndex < currentPoly.Vertices.Num(); ++vertIndex )
		{
			FVector3f newVertex = currentPoly.Vertices[vertIndex];
			newPoly.Vertices.Add( newVertex );	
		}
		CachedPolys->Element.Add(newPoly);
	}
}
 

void UGeomModifier::RestoreBrushState()
{
	FEdModeGeometry* GeomMode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	ABrush* BuilderBrush = GeomMode->GetWorld()->GetDefaultBrush();

	//Remove all of the current polys
	BuilderBrush->Brush->Polys->Element.Empty();

	//Add all of the cached polys
	for( int32 polyIndex = 0 ; polyIndex < CachedPolys->Element.Num() ; polyIndex++ )
	{
		BuilderBrush->Brush->Polys->Element.Push(CachedPolys->Element[polyIndex]);
	}

	BuilderBrush->Brush->BuildBound();
	
	BuilderBrush->ReregisterAllComponents();

	GeomMode->FinalizeSourceData();
	GeomMode->GetFromSource();

	GEditor->SelectNone( true, true );

	GEditor->RedrawLevelEditingViewports(true);

	//Tell the user what just happened
	FMessageDialog::Debugf(LOCTEXT("InvalidBrushState", "Invalid brush state could fail to triangulate.  Reverting to previous state."));
}


bool UGeomModifier::DoEdgesOverlap()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	//Loop through all of the geometry objects
	for( FEdModeGeometry::TGeomObjectIterator itor( mode->GeomObjectItor() ) ; itor ; ++itor )
	{
		FGeomObjectPtr geomObject = *itor;

		//Loop through all of the edges
		for( int32 edgeIndex1 = 0 ; edgeIndex1 < geomObject->EdgePool.Num() ; ++edgeIndex1 )
		{
			FGeomEdge* edge1 = &geomObject->EdgePool[edgeIndex1];

			for( int32 edgeIndex2 = 0 ; edgeIndex2 < geomObject->EdgePool.Num() ; ++edgeIndex2 )
			{
				FGeomEdge* edge2 = &geomObject->EdgePool[edgeIndex2];
				//Don't compare an edge with itself
				if( !(edge1->IsSameEdge(*edge2)) )
				{
					FVector closestPoint1, closestPoint2;
					FVector edge1Vert1 = (FVector)geomObject->VertexPool[ edge1->VertexIndices[0] ];
					FVector edge2Vert1 = (FVector)geomObject->VertexPool[ edge2->VertexIndices[0] ];
					FVector edge1Vert2 = (FVector)geomObject->VertexPool[ edge1->VertexIndices[1] ];
					FVector edge2Vert2 = (FVector)geomObject->VertexPool[ edge2->VertexIndices[1] ];

					//Find the distance between the two segments
					FMath::SegmentDistToSegment( edge1Vert1, edge1Vert2, edge2Vert1, edge2Vert2, closestPoint1, closestPoint2 );

					if ( (closestPoint1.Equals(closestPoint2)) )
					{
						//Identical closest points indicates that lines cross
						bool bSharedVertex =  ((edge1Vert1.Equals(edge2Vert1)) || (edge1Vert1.Equals(edge2Vert2))
											|| (edge1Vert2.Equals(edge2Vert1)) || (edge1Vert2.Equals(edge2Vert2)));

						// Edges along the same line are exempt
						if (  !bSharedVertex )
						{
							bool bIntersectionIsVert = ((edge1Vert1.Equals(closestPoint2, THRESH_POINTS_ARE_SAME)) || (edge1Vert2.Equals(closestPoint2, THRESH_POINTS_ARE_SAME))
												||  (edge2Vert1.Equals(closestPoint2, THRESH_POINTS_ARE_SAME )) || (edge2Vert2.Equals(closestPoint2, THRESH_POINTS_ARE_SAME)) );

							// Edges intersecting at a vertex are exempt
							if ( !bIntersectionIsVert )
							{
								// Edges cross.  The shape drawn with this brush will likely be undesireable
								return true;
							}
						}
					}
				}
			}
		}
	}

	return false;
}

void UGeomModifier::UpdatePivotOffset()
{
	if (!GetDefault<ULevelEditorMiscSettings>()->bAutoMoveBSPPivotOffset)
	{
		return;
	}

	FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	for (FEdModeGeometry::TGeomObjectIterator It(Mode->GeomObjectItor()); It; ++It)
	{
		FGeomObjectPtr GeomObject = *It;
		ABrush* Brush = GeomObject->GetActualBrush();

		TSet<FVector> UniqueVertices;
		FVector VertexCenter = FVector::ZeroVector;

		if (Brush->Brush && Brush->Brush->Polys)
		{
			for (const auto& Element : Brush->Brush->Polys->Element)
			{
				for (const auto& Vertex : Element.Vertices)
				{
					UniqueVertices.Add((FVector)Vertex);
				}
			}

			for (const auto& Vertex : UniqueVertices)
			{
				VertexCenter += Vertex;
			}

			if (UniqueVertices.Num() > 0)
			{
				VertexCenter /= UniqueVertices.Num();
			}
		}

		Brush->SetPivotOffset(VertexCenter);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
//	Transaction tracking.
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace {
	/**
	 * @return		The shared transaction object used by 
	 */
	static FScopedTransaction*& StaticTransaction()
	{
		static FScopedTransaction* STransaction = NULL;
		return STransaction;
	}

	/**
	 * Ends the outstanding transaction, if one exists.
	 */
	static void EndTransaction()
	{
		delete StaticTransaction();
		StaticTransaction() = NULL;
	}

	/**
	 * Begins a new transaction, if no outstanding transaction exists.
	 */
	static void BeginTransaction(const FText& Description)
	{
		if ( !StaticTransaction() )
		{
			StaticTransaction() = new FScopedTransaction( Description );
		}
	}
} // namespace


void UGeomModifier::StartTrans()
{
	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return;
	}

	StoreAllCurrentGeomSelections();

	// Start the transaction.
	BeginTransaction( FText::Format( NSLOCTEXT("UnrealEd", "Modifier_F", "Modifier [{0}]"), GetModifierDescription() ) );

	// Mark all selected brushes as modified.
	FEdModeGeometry* CurMode = static_cast<FEdModeGeometry*>( GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry) );
	for( FEdModeGeometry::TGeomObjectIterator Itor( CurMode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;
		ABrush* Actor = go->GetActualBrush();

		Actor->Modify(false);
	}
}


void UGeomModifier::EndTrans()
{
	EndTransaction();
}


void UGeomModifier::StoreCurrentGeomSelections( TArray<struct FGeomSelection>& SelectionArray, FGeomObjectPtr go ) 
{
	SelectionArray.Empty();
	FGeomSelection* gs = NULL;

	for( int32 v = 0 ; v < go->VertexPool.Num() ; ++v )
	{
		FGeomVertex* gv = &go->VertexPool[v];
		if( gv->IsSelected() )
		{
			gs = new( SelectionArray )FGeomSelection;
			gs->Type = GS_Vertex;
			gs->Index = v;
			gs->SelectionIndex = gv->GetSelectionIndex();
		}
	}
	for( int32 e = 0 ; e < go->EdgePool.Num() ; ++e )
	{
		FGeomEdge* ge = &go->EdgePool[e];
		if( ge->IsSelected() )
		{
			gs = new( SelectionArray )FGeomSelection;
			gs->Type = GS_Edge;
			gs->Index = e;
			gs->SelectionIndex = ge->GetSelectionIndex();
		}
	}
	for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
	{
		FGeomPoly* gp = &go->PolyPool[p];
		if( gp->IsSelected() )
		{
			gs = new( SelectionArray )FGeomSelection;
			gs->Type = GS_Poly;
			gs->Index = p;
			gs->SelectionIndex = gp->GetSelectionIndex();
		}
	}
}

void UGeomModifier::StoreAllCurrentGeomSelections()
{
	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return;
	}

	FEdModeGeometry* CurMode = static_cast<FEdModeGeometry*>( GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry) );

	// Record the current selection list into the selected brushes.
	for( FEdModeGeometry::TGeomObjectIterator Itor( CurMode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;
		go->CompileSelectionOrder();

		ABrush* Actor = go->GetActualBrush();

		StoreCurrentGeomSelections( Actor->SavedSelections , go );
	}
}


UGeomModifier_Edit::UGeomModifier_Edit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Edit", "Edit");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Edit", "Tooltip", "Select faces, edges, or vertices directly in the viewport.  Use the manipulator to translate, rotate or scale existing geometry.");
	ToolbarIconName = TEXT("GeometryMode.EditTool");
}

bool UGeomModifier_Edit::SupportsCurrentSelection()
{
	return GEditor->GetSelectedActors()->CountSelections<ABrush>() > 0;
}

bool UGeomModifier_Edit::InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale)
{
	if( UGeomModifier::InputDelta( InViewportClient, InViewport, InDrag, InRot, InScale ) )
	{
		return true;
	}

	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return false;
	}

	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	FModeTool_GeometryModify* tool = (FModeTool_GeometryModify*)mode->GetCurrentTool();

	TArray<FGeomVertex*> UniqueVertexList;

	/**
	* All geometry objects can be manipulated by transforming the vertices that make
	* them up.  So based on the type of thing we're editing, we need to dig for those
	* vertices a little differently.
	*/

	// Only permit scaling if there is exactly one selected poly. Make a note of it here
	FGeomPoly* SelectedPoly = nullptr;
	int32 NumSelectedPolys = 0;

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
		{
			FGeomPoly* gp = &go->PolyPool[p];
			if( gp->IsSelected() )
			{
				SelectedPoly = gp;
				NumSelectedPolys++;

				for( int32 e = 0 ; e < gp->EdgeIndices.Num() ; ++e )
				{
					FGeomEdge* ge = &go->EdgePool[ gp->EdgeIndices[e] ];
					UniqueVertexList.AddUnique( &go->VertexPool[ ge->VertexIndices[0] ] );
					UniqueVertexList.AddUnique( &go->VertexPool[ ge->VertexIndices[1] ] );
				}
			}
		}

		for( int32 e = 0 ; e < go->EdgePool.Num() ; ++e )
		{
			FGeomEdge* ge = &go->EdgePool[e];
			if( ge->IsSelected() )
			{
				UniqueVertexList.AddUnique( &go->VertexPool[ ge->VertexIndices[0] ] );
				UniqueVertexList.AddUnique( &go->VertexPool[ ge->VertexIndices[1] ] );
			}
		}

		for( int32 v = 0 ; v < go->VertexPool.Num() ; ++v )
		{
			FGeomVertex* gv = &go->VertexPool[v];
			if( gv->IsSelected() )
			{
				UniqueVertexList.AddUnique( gv );
			}
		}
	}

	// If we didn't move any vertices, then tell the caller that we didn't handle the input.
	// This allows LDs to drag brushes around in geometry mode as long as no geometry
	// objects are selected.
	if( !UniqueVertexList.Num() )
	{
		return false;
	}

	const bool bShiftPressed = InViewportClient->IsShiftPressed();

	// If we're trying to rotate vertices, only allow that if Shift is held down.  This just makes it easier
	// to rotate brushes around while working in geometry mode, since you rarely ever want to rotate vertices
	FRotator FinalRot = InRot;
	if( !bShiftPressed )
	{
		FinalRot = FRotator::ZeroRotator;
	}

	if( InDrag.IsZero() && FinalRot.IsZero() && InScale.IsZero() )
	{
		// No change, but handled
		return true;
	}

	StartTrans();

	// Let tool know that some modification has actually taken place
	tool->bGeomModified = true;

	/**
	* Scaling needs to know the bounding box for the selected verts, so generate that before looping.
	*/
	FBox VertBBox(ForceInit);

	for( int32 x = 0 ; x < UniqueVertexList.Num() ; ++x )
	{
		VertBBox += (FVector)*UniqueVertexList[x];
	}

	FVector BBoxExtent = VertBBox.GetExtent();

	/**
	 * We first generate a list of unique vertices and then transform that list
	 * in one shot.  This prevents vertices from being touched more than once (which
	 * would result in them transforming x times as fast as others).
	 */
	for( int32 x = 0 ; x < UniqueVertexList.Num() ; ++x )
	{
		FGeomVertex* vtx = UniqueVertexList[x];
		const ABrush* Brush = vtx->GetParentObject()->GetActualBrush();

		// Translate

		if (!InDrag.IsZero())
		{
			*vtx += (FVector3f)Brush->ActorToWorld().InverseTransformVector(InDrag);
		}

		// Rotate

		if( !FinalRot.IsZero() )
		{
			const FRotationMatrix Matrix( FinalRot );

			FVector Wk( vtx->X, vtx->Y, vtx->Z );
			Wk = Brush->ActorToWorld().TransformPosition(Wk);
			Wk -= GLevelEditorModeTools().PivotLocation;
			Wk = Matrix.TransformPosition( Wk );
			Wk += GLevelEditorModeTools().PivotLocation;
			*vtx = (FVector3f)Brush->ActorToWorld().InverseTransformPosition((FVector)Wk);
		}

		// Scale

		if( !InScale.IsZero() && NumSelectedPolys == 1)
		{
			// This is a quick fix for now.
			// Scale needs to know the surface normal of the polys selected (and scale only makes sense on polys, not edges or verts),
			// so remember one selected poly and use that transform.
			// Since scaling is relative to the pivot, it would actually be horrible scaling multiple polys at once anyway.
			const FScaleMatrix Matrix(InScale + 1.0f);
			const FVector PivotInModelSpace = Brush->ActorToWorld().InverseTransformPosition(GLevelEditorModeTools().PivotLocation);
			const FRotationMatrix GeomBaseTransform = FRotationMatrix(SelectedPoly->GetWidgetRotation());

			FVector Wk(vtx->X, vtx->Y, vtx->Z);
			Wk -= PivotInModelSpace;
			Wk = GeomBaseTransform.TransformPosition(Wk);
			Wk = Matrix.TransformPosition(Wk);
			Wk = GeomBaseTransform.InverseTransformPosition(Wk);
			Wk += PivotInModelSpace;
			*vtx = (FVector3f)Wk;
		}
	}
	
	if( DoEdgesOverlap() )
	{
		//Two edges overlap, which causes triangulation problems, so move the vertices back to their previous location
		for( int32 x = 0 ; x < UniqueVertexList.Num() ; ++x )
		{
			FGeomVertex* vtx = UniqueVertexList[x];
			const ABrush* Brush = vtx->GetParentObject()->GetActualBrush();
			*vtx -= (FVector3f)Brush->ActorToWorld().InverseTransformVector(InDrag);	// LWC_TODO: Precision Loss
		}
		
		GLevelEditorModeTools().PivotLocation -= InDrag;
		GLevelEditorModeTools().SnappedLocation -= InDrag;
	}

	const bool bIsCtrlPressed = InViewportClient->IsCtrlPressed();
	const bool bIsAltPressed = InViewportClient->IsAltPressed();

	if(!InDrag.IsZero() && bShiftPressed && bIsCtrlPressed && !bIsAltPressed)
	{
		FVector CameraDelta(InDrag);

		// Only apply camera speed modifiers to the drag if we aren't zooming in an ortho viewport.
		if( !InViewportClient->IsOrtho() || !(InViewport->KeyState(EKeys::LeftMouseButton) && InViewport->KeyState(EKeys::RightMouseButton)) )
		{
			const float CameraSpeed = InViewportClient->GetCameraSpeed();
			CameraDelta *= CameraSpeed;
		}

		InViewportClient->MoveViewportCamera( CameraDelta, InRot );
	}

	EndTrans();
	bPendingPivotOffsetUpdate = true;
	GEditor->RedrawLevelEditingViewports(true);

	return true;
}

/*------------------------------------------------------------------------------
	UGeomModifier_Extrude
------------------------------------------------------------------------------*/
UGeomModifier_Extrude::UGeomModifier_Extrude(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Extrude", "Extrude");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Extrude", "Tooltip", "Moves the selected geometry element forward, creating new geometry behind it if necessary.");
	Length = 16;
	Segments = 1;
	ToolbarIconName = TEXT("GeometryMode.ExtrudeTool");
}

bool UGeomModifier_Extrude::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	const bool bGetRawValue = true;
	const bool bIsLocalCoords = GLevelEditorModeTools().GetCoordSystem(bGetRawValue) == COORD_Local;

	if (!bIsLocalCoords)
	{
		// Before the modal dialog has popped up, force tracking to stop and reset the focus
		InViewportClient->LostFocus(InViewport);
		InViewportClient->ReceivedFocus(InViewport);
		CheckCoordinatesMode();
	}

	if (!bIsLocalCoords || Mode->GetCurrentWidgetAxis() != EAxisList::X)
	{
		InDrag = FVector::ZeroVector;
		InRot = FRotator::ZeroRotator;
		InScale = FVector::ZeroVector;
		return false;
	}

	return Super::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
}


bool UGeomModifier_Extrude::SupportsCurrentSelection()
{
	FEdModeGeometry* Mode = (FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && Mode->HavePolygonsSelected();
}

void UGeomModifier_Extrude::WasActivated()
{
	// Extrude requires a local coordinate system to work properly so automatically enable
	// that here while saving the current coordinate system for restoration later.
	const bool bGetRawValue = true;
	SaveCoordSystem = GLevelEditorModeTools().GetCoordSystem(bGetRawValue);
	
	CheckCoordinatesMode();

	GEditor->RedrawLevelEditingViewports(true);
}

void UGeomModifier_Extrude::WasDeactivated()
{
	// When the user leaves this modifier, restore their old coordinate system.
	GLevelEditorModeTools().SetCoordSystem((ECoordSystem)SaveCoordSystem);

	GEditor->RedrawLevelEditingViewports(true);
}

void UGeomModifier_Extrude::CheckCoordinatesMode() 
{
	const bool bGetRawValue = true;	
	if( GLevelEditorModeTools().GetCoordSystem(bGetRawValue) != COORD_Local )
	{
		FSuppressableWarningDialog::FSetupInfo Info( LOCTEXT("ExtrudeCoordinateWarningBody","Extrude only works with Local Coordinates System"), LOCTEXT("ExtrudeCoordinateWarningTitle","Extrude Coordinates Mode Warning"), "ExtrudeCoordsWarning" );
		Info.ConfirmText = LOCTEXT( "Close", "Close");

		FSuppressableWarningDialog WarnAboutCoordinatesSystem( Info );
		WarnAboutCoordinatesSystem.ShowModal();			
		GLevelEditorModeTools().SetCoordSystem(COORD_Local);
	}
}


void UGeomModifier_Extrude::Initialize()
{
	//the user may have changed the mode AFTER going into extrude - double check its LOCAL not WORLD
	CheckCoordinatesMode();
	const int32 GridSize = static_cast<int32>(GEditor->GetGridSize());
	Apply( GridSize , 1 );
}

bool UGeomModifier_Extrude::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	// When applying via the keyboard, we force the local coordinate system.
	const bool bGetRawValue = true;
	const ECoordSystem SaveCS = GLevelEditorModeTools().GetCoordSystem(bGetRawValue);
	GLevelEditorModeTools().SetCoordSystem(COORD_Local);

	//GApp->DlgGeometryTools->PropertyWindow->FinalizeValues();
	Apply( Length, Segments );

	// Restore the coordinate system.
	GLevelEditorModeTools().SetCoordSystem(SaveCS);

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp
	bPendingPivotOffsetUpdate = true;

	return true;
}

void ExtrudePolygonGroup( ABrush* InBrush, FVector3f InGroupNormal, int32 InStartOffset, int32 InLength, TArray<FPoly>& InPolygonGroup )
{
	TArray< TArray<FVector3f> > Windings;

	FPoly::GetOutsideWindings( InBrush, InPolygonGroup, Windings );

	for( int32 w = 0 ; w < Windings.Num() ; ++w )
	{
		TArray<FVector3f>* WindingVerts = &Windings[w];

		FVector3f Offset = InGroupNormal * InLength;
		FVector3f StartOffset = InGroupNormal * InStartOffset;

		for( int32 v = 0 ; v < WindingVerts->Num() ; ++v )
		{
			FVector3f vtx0 = StartOffset + (*WindingVerts)[ v ];
			FVector3f vtx1 = StartOffset + (*WindingVerts)[ v ] + Offset;
			FVector3f vtx2 = StartOffset + (*WindingVerts)[ (v + 1) % WindingVerts->Num() ] + Offset;
			FVector3f vtx3 = StartOffset + (*WindingVerts)[ (v + 1) % WindingVerts->Num() ];

			FPoly NewPoly;
			NewPoly.Init();
			NewPoly.Base = (FVector3f)InBrush->GetActorLocation();	// LWC_TODO: Precision Loss

			NewPoly.Vertices.Add( vtx1 );
			NewPoly.Vertices.Add( vtx0 );
			NewPoly.Vertices.Add( vtx3 );
			NewPoly.Vertices.Add( vtx2 );

			if( NewPoly.Finalize( InBrush, 1 ) == 0 )
			{
				InBrush->Brush->Polys->Element.Add( NewPoly );
			}
		}
	}
}

void UGeomModifier_Extrude::Apply(int32 InLength, int32 InSegments)
{
	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return;
	}

	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	// Force user input to be valid

	InLength = FMath::Max( 1, InLength );
	InSegments = FMath::Max( 1, InSegments );

	//

	TArray<int32> SavedSelectionIndices;

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;
		ABrush* Brush = go->GetActualBrush();

		go->SendToSource();

		TArray<FPoly> Polygons;

		for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
		{
			FGeomPoly* gp = &go->PolyPool[p];

			FVector Normal = Brush->GetActorQuat().Inverse().RotateVector( mode->GetWidgetNormalFromCurrentAxis( gp ) );

			if( gp->IsSelected() )
			{
				SavedSelectionIndices.Add( p );

				FPoly* Poly = gp->GetActualPoly();

				Polygons.Add( *Poly );

				// Move the existing poly along the normal by InLength units.

				for( int32 v = 0 ; v < Poly->Vertices.Num() ; ++v )
				{
					FVector3f* vtx = &Poly->Vertices[v];

					*vtx += FVector3f(Normal * (InLength * InSegments));
				}

				Poly->Base += FVector3f(Normal * (InLength * InSegments));
			}
		}

		if( Polygons.Num() )
		{
			struct FCompareFPolyNormal
			{
				FORCEINLINE bool operator()( const FPoly& A, const FPoly& B ) const
				{
					return (B.Normal - A.Normal).SizeSquared() < 0.f;
				}
			};
			Polygons.Sort( FCompareFPolyNormal() );

			FVector3f NormalCompare = FVector3f::ZeroVector;
			TArray<FPoly> PolygonGroup;

			for( int32 p = 0 ; p < Polygons.Num() ; ++p )
			{
				FPoly* Poly = &Polygons[p];

				if( p == 0 )
				{
					NormalCompare = Poly->Normal;
				}

				if( NormalCompare.Equals( Poly->Normal ) )
				{
					PolygonGroup.Add( *Poly );
				}
				else
				{
					if( PolygonGroup.Num() )
					{
						for( int32 s = 0 ; s < InSegments ; ++s )
						{
							ExtrudePolygonGroup( Brush, NormalCompare, InLength * s, InLength, PolygonGroup );
						}
					}

					NormalCompare = Poly->Normal;
					PolygonGroup.Empty();
					PolygonGroup.Add( *Poly );
				}
			}

			if( PolygonGroup.Num() )
			{
				for( int32 s = 0 ; s < InSegments ; ++s )
				{
					ExtrudePolygonGroup( Brush, NormalCompare, InLength * s, InLength, PolygonGroup );
				}
			}
		}

		go->FinalizeSourceData();
		go->GetFromSource();

		for( int32 x = 0 ; x < SavedSelectionIndices.Num() ; ++x )
		{
			FGeomPoly* Poly = &go->PolyPool[ SavedSelectionIndices[x] ];
			Poly->Select(1);
		}
	}
}

UGeomModifier_Lathe::UGeomModifier_Lathe(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Lathe", "Lathe");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Lathe", "Tooltip", "Take Brush Shapes (created via the Pen Tool with CreateBrushShape enabled) and revolve them around the pivot point to create curved brushes.  Create a pivot using ALT + MMB Drag, then switch to a Top-Down Viewport and hit Apply.");
	Axis = EAxis::Y;
	TotalSegments = 16;
	Segments = 4;
	AlignToSide = false;
	ToolbarIconName = TEXT("GeometryMode.LatheTool");
}


bool UGeomModifier_Lathe::SupportsCurrentSelection()
{
	// Lathe mode requires ABrushShape actors to be selected.

	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		ABrush* Brush = Cast<ABrush>( *It );

		if( Brush && Brush->IsBrushShape() )
		{
			return true;
		}
	}

	return false;
}

void UGeomModifier_Lathe::Initialize()
{
}


bool UGeomModifier_Lathe::OnApply()
{
	//FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetCurrentMode();

	//GApp->DlgGeometryTools->PropertyWindow->FinalizeValues();
	Apply( TotalSegments, Segments, Axis );

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	bPendingPivotOffsetUpdate = true;
	return true;
}

void UGeomModifier_Lathe::Apply( int32 InTotalSegments, int32 InSegments, EAxis::Type InAxis )
{
	if( !GLevelEditorModeTools().IsModeActive(FGeometryEditingModes::EM_Geometry) )
	{
		return;
	}

	// Determine the axis from the active ortho viewport

	if ( !GLastKeyLevelEditingViewportClient || !GLastKeyLevelEditingViewportClient->IsOrtho() )
	{
		return;
	}

	//Save the brush state in case a bogus shape is generated
	CacheBrushState();

	switch( GLastKeyLevelEditingViewportClient->ViewportType )
	{
		case LVT_OrthoXZ:
			Axis = EAxis::X;
			break;

		case LVT_OrthoXY:
			Axis = EAxis::Y;
			break;

		case LVT_OrthoYZ:
			Axis = EAxis::Z;
			break;
	}

	FEdModeGeometry* GeomMode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	InTotalSegments = FMath::Max( 3, InTotalSegments );
	InSegments = FMath::Max( 1, InSegments );

	if( InSegments > InTotalSegments )
	{
		InTotalSegments = InSegments;
	}

	// We will be replacing the builder brush, so get it prepped.

	ABrush* BuilderBrush = GeomMode->GetWorld()->GetDefaultBrush();

	BuilderBrush->SetActorLocation(GeomMode->GetWidgetLocation(), false);
	BuilderBrush->SetPivotOffset(FVector::ZeroVector);
	BuilderBrush->SetFlags( RF_Transactional );
	BuilderBrush->Brush->Polys->Element.Empty();

	// Ensure the builder brush is unhidden.
	BuilderBrush->SetHidden(false);
	BuilderBrush->bHiddenEdLayer = false;
	BuilderBrush->SetIsTemporarilyHiddenInEditor( false );

	// Some convenience flags
	bool bNeedCaps = (InSegments < InTotalSegments);

	// Lathe every selected ABrushShape actor into the builder brush

	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		TArray<FEdge> EdgePool;

		ABrushShape* BrushShape = Cast<ABrushShape>( *It );

		if( BrushShape )
		{
			if( BrushShape->Brush->Polys->Element.Num() < 1 )
			{
				continue;
			}

			TArray< TArray<FVector3f> > Windings;
			FPoly::GetOutsideWindings( BrushShape, BrushShape->Brush->Polys->Element, Windings );

			FVector delta = GeomMode->GetWidgetLocation() - BrushShape->GetActorLocation();

			//
			// Let's lathe...
			//

			// Build up an array of vertices that represents the entire lathe.

			float AngleStep = 360.f / (float)InTotalSegments;
			float Angle = 0;

			for( int32 w = 0 ; w < Windings.Num() ; ++w )
			{
				TArray<FVector3f>* WindingVerts = &Windings[w];

				TArray<FVector> ShapeVertices;

				for( int32 s = 0 ; s < (InSegments + 1 + (AlignToSide?1:0)) ; ++s )
				{
					FRotator rot = FRotator( 0, Angle, 0 );
					if( Axis == EAxis::X )
					{
						rot = FRotator( Angle, 0, 0 );
					}
					else if( Axis == EAxis::Z )
					{
						rot = FRotator( 0, 0, Angle );
					}

					FRotationMatrix RotationMatrix( rot );

					for( int32 e = 0 ; e < WindingVerts->Num() ; ++e )
					{
						FVector vtx = (FVector)(*WindingVerts)[e] - delta - BrushShape->GetPivotOffset();

						vtx = RotationMatrix.TransformPosition( vtx );

						ShapeVertices.Add( vtx );
					}

					if( AlignToSide && (s == 0 || s == InSegments) )
					{
						Angle += AngleStep / 2.0f;
					}
					else
					{
						Angle += AngleStep;
					}
				}

				int32 NumVertsInShape = WindingVerts->Num();

				for( int32 s = 0 ; s < InSegments + (AlignToSide?1:0) ; ++s )
				{
					int32 BaseIdx = s * WindingVerts->Num();

					for( int32 v = 0 ; v < NumVertsInShape ; ++v )
					{
						FVector vtx0 = ShapeVertices[ BaseIdx + v ];
						FVector vtx1 = ShapeVertices[ BaseIdx + NumVertsInShape + v ];
						FVector vtx2 = ShapeVertices[ BaseIdx + NumVertsInShape + ((v + 1) % NumVertsInShape) ];
						FVector vtx3 = ShapeVertices[ BaseIdx + ((v + 1) % NumVertsInShape) ];

						FPoly NewPoly;
						NewPoly.Init();
						NewPoly.Base = (FVector3f)BuilderBrush->GetActorLocation();

						NewPoly.Vertices.Add( (FVector3f)vtx0 );
						NewPoly.Vertices.Add( (FVector3f)vtx1 );
						NewPoly.Vertices.Add( (FVector3f)vtx2 );
						NewPoly.Vertices.Add( (FVector3f)vtx3 );

						if( NewPoly.Finalize( BuilderBrush, 1 ) == 0 )
						{
							BuilderBrush->Brush->Polys->Element.Add( NewPoly );
						}
						
					}
				}
			}

			// Create start/end capping polygons if they are necessary

			if( bNeedCaps )
			{
				for( int32 w = 0 ; w < Windings.Num() ; ++w )
				{
					TArray<FVector3f>* WindingVerts = &Windings[w];

					//
					// Create the start cap
					//

					FPoly Poly;
					Poly.Init();
					Poly.Base = (FVector3f)BrushShape->GetActorLocation();

					// Add the verts from the shape

					for( int32 v = 0 ; v < WindingVerts->Num() ; ++v )
					{
						Poly.Vertices.Add(FVector3f((FVector)(*WindingVerts)[v] - delta - BrushShape->GetPivotOffset()));
					}

					Poly.Finalize( BuilderBrush, 1 );

					// Break the shape down into convex shapes.

					TArray<FPoly> Polygons;
					Poly.Triangulate( BuilderBrush, Polygons );
					FPoly::OptimizeIntoConvexPolys( BuilderBrush, Polygons );

					// Add the resulting convex polygons into the brush

					for( int32 p = 0 ; p < Polygons.Num() ; ++p )
					{
						FPoly Polygon = Polygons[p];

						if (Polygon.Finalize(BuilderBrush, 1) == 0)
						{
							BuilderBrush->Brush->Polys->Element.Add(Polygon);
						}
					}

					//
					// Create the end cap
					//

					Poly.Init();
					Poly.Base = (FVector3f)BrushShape->GetActorLocation();	// LWC_TODO: Precision Loss

					// Add the verts from the shape

					FRotator3f rot = FRotator3f( 0, AngleStep * InSegments, 0 );
					if( Axis == EAxis::X )
					{
						rot = FRotator3f( AngleStep * InSegments, 0, 0 );
					}
					else if( Axis == EAxis::Z )
					{
						rot = FRotator3f( 0, 0, AngleStep * InSegments );
					}

					FRotationMatrix44f RotationMatrix( rot );

					for( int32 v = 0 ; v < WindingVerts->Num() ; ++v )
					{
						Poly.Vertices.Add(RotationMatrix.TransformPosition(FVector3f((FVector)(*WindingVerts)[v] - delta - BrushShape->GetPivotOffset())));
					}

					Poly.Finalize( BuilderBrush, 1 );

					// Break the shape down into convex shapes.

					Polygons.Empty();
					Poly.Triangulate( BuilderBrush, Polygons );
					FPoly::OptimizeIntoConvexPolys( BuilderBrush, Polygons );

					// Add the resulting convex polygons into the brush

					for( int32 p = 0 ; p < Polygons.Num() ; ++p )
					{
						FPoly Polygon = Polygons[p];
						Polygon.Reverse();

						if (Polygon.Finalize(BuilderBrush, 1) == 0)
						{
							BuilderBrush->Brush->Polys->Element.Add(Polygon);
						}
					}
				}
			}
		}
	}

	// Finalize the builder brush

	BuilderBrush->Brush->BuildBound();

	BuilderBrush->ReregisterAllComponents();

	GeomMode->FinalizeSourceData();
	GeomMode->GetFromSource();

	GEditor->SelectNone( true, true );
	GEditor->SelectActor( BuilderBrush, true, true );

	if( DoEdgesOverlap() )
	{//Overlapping edges yielded an invalid brush state
		RestoreBrushState();
	}
	else
	{
		GEditor->RedrawLevelEditingViewports(true);
	}

	// Create additive brush from builder brush
	GEditor->Exec(GeomMode->GetWorld(), TEXT("BRUSH ADD SELECTNEWBRUSH"));

	// Deselect & hide builder brush
	BuilderBrush->SetIsTemporarilyHiddenInEditor(true);
	GEditor->SelectActor(BuilderBrush, false, false);
}

/*------------------------------------------------------------------------------
	UGeomModifier_Pen
------------------------------------------------------------------------------*/
UGeomModifier_Pen::UGeomModifier_Pen(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Pen", "Pen");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Pen", "Tooltip", "Create new geometry by drawing the vertices directly into an orthographic viewport. Press space bar to place a vertex, and Enter to close the polygon.");
	bCreateBrushShape = false;
	bAutoExtrude = true;
	ExtrudeDepth = 256;
	bCreateConvexPolygons = true;
	ToolbarIconName = TEXT("GeometryMode.PenTool");
}

/**
* Pen can work always
*/
bool UGeomModifier_Pen::SupportsCurrentSelection()
{
	return true;
}

/**
* Gives the modifier a chance to initialize it's internal state when activated.
*/

void UGeomModifier_Pen::WasActivated()
{
	ShapeVertices.Empty();
}

/**
* Implements the modifier application.
*/
bool UGeomModifier_Pen::OnApply()
{
	Apply();

	bPendingPivotOffsetUpdate = true;
	return true;
}

void UGeomModifier_Pen::Apply()
{
	if( ShapeVertices.Num() > 2 )
	{
		FEdModeGeometry* GeomMode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
		ABrush* ResultingBrush = GeomMode->GetWorld()->GetDefaultBrush();
		ABrush* BuilderBrush = GeomMode->GetWorld()->GetDefaultBrush();

		// Move all the vertices that the user placed to the same "height" as the builder brush, based on
		// viewport orientation.  This is preferable to always creating the new builder brush at height zero.
		
		for( int32 v = 0 ; v < ShapeVertices.Num() ; ++v )
		{
			FVector* vtx = &ShapeVertices[v];

			switch( GLastKeyLevelEditingViewportClient->ViewportType )
			{
			case LVT_OrthoXY:
				vtx->Z = BuilderBrush->GetActorLocation().Z;
				break;

			case LVT_OrthoXZ:
				vtx->Y = BuilderBrush->GetActorLocation().Y;
				break;

			case LVT_OrthoYZ:
				vtx->X = BuilderBrush->GetActorLocation().X;
				break;
			}
		}

		// Generate center location from the shape's center
		FBox WorldBounds(ShapeVertices.GetData(), ShapeVertices.Num());
		FVector BaseLocation = WorldBounds.GetCenter();

		//create a scoped transaction so that we can undo the creation/modification		
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushSet", "Brush Set") );

		//if we are creating a brush we need to first create an actor for it
		if( bCreateBrushShape )
		{	
			// Create a shape brush instead of modifying the builder brush
			ResultingBrush = BuilderBrush->GetWorld()->SpawnActor<ABrushShape>(BaseLocation, FRotator::ZeroRotator);
			
			ResultingBrush->PreEditChange(NULL);
			// It's OK to create an empty brush here as we are going to re-create the polys anyway.
			FBSPOps::csgCopyBrush( ResultingBrush, BuilderBrush, PF_DefaultFlags,  BuilderBrush->GetFlags(), true, true, true );
			ResultingBrush->PostEditChange();			

		}
		else
		{
			ResultingBrush = FBSPOps::csgAddOperation( BuilderBrush, PF_DefaultFlags, Brush_Add );
			if (ResultingBrush == nullptr)
			{
				return;
			}

			if (ResultingBrush->GetBrushBuilder())
			{
				FActorLabelUtilities::SetActorLabelUnique(ResultingBrush, FText::Format(NSLOCTEXT("BSPBrushOps", "BrushName", "{0} Brush"), FText::FromString(ResultingBrush->GetBrushBuilder()->GetClass()->GetDescription())).ToString());
			}
		}

		// Make sure the graphics engine isn't busy rendering this geometry before we go and modify it
		FlushRenderingCommands();
		
		ResultingBrush->SetActorLocation(BaseLocation, false);
		ResultingBrush->SetPivotOffset(FVector::ZeroVector);
		ResultingBrush->SetFlags( RF_Transactional );
		ResultingBrush->Brush->Polys->Element.Empty();

		// Ensure the brush is unhidden.
		ResultingBrush->SetHidden(false);
		ResultingBrush->bHiddenEdLayer = false;
		ResultingBrush->SetIsTemporarilyHiddenInEditor( false );

		FPoly Poly;
		Poly.Init();
		Poly.Base = (FVector3f)BaseLocation;

		for( int32 v = 0 ; v < ShapeVertices.Num() ; ++v )
		{
			new(Poly.Vertices) FVector3f( ShapeVertices[v] - BaseLocation );
		}

		if( Poly.Finalize( ResultingBrush, 1 ) == 0 )
		{
			// Break the shape down into triangles.

			TArray<FPoly> Triangles;
			Poly.Triangulate( ResultingBrush, Triangles );

			TArray<FPoly> Polygons = Triangles;

			// Optionally, optimize the resulting triangles into convex polygons.

			if( bCreateConvexPolygons )
			{
				FPoly::OptimizeIntoConvexPolys( ResultingBrush, Polygons );
			}

			// If the user isn't creating an ABrushShape, then carry on adding the sides and bottom face
			// If the user wants a full brush created, add the rest of the polys

			if( !bCreateBrushShape && bAutoExtrude && ExtrudeDepth > 0 )
			{
				// Extruding along delta
				FVector HalfDelta;

				// Create another set of polygons that will represent the bottom face

				for( int32 p = 0 ; p < Polygons.Num() ; ++p )
				{
					FPoly poly = Polygons[p];

					if (p == 0)
					{
						HalfDelta = 0.5f * (FVector)poly.Normal * ExtrudeDepth;
					}

					if( poly.Finalize( ResultingBrush, 0 ) == 0 )
					{
						for( int32 v = 0 ; v < poly.Vertices.Num() ; ++v )
						{

							FVector3f* vtx = &poly.Vertices[v];
							*vtx += (FVector3f)HalfDelta;
						}

						new(ResultingBrush->Brush->Polys->Element)FPoly( poly );
					}

					poly.Reverse();

					if( poly.Finalize( ResultingBrush, 0 ) == 0 )
					{
						for( int32 v = 0 ; v < poly.Vertices.Num() ; ++v )
						{
							FVector3f* vtx = &poly.Vertices[v];
							*vtx -= 2.0f * (FVector3f)HalfDelta;
						}

						new(ResultingBrush->Brush->Polys->Element)FPoly( poly );
					}
				}

				// Create the polygons that make up the sides of the brush

				if( Polygons.Num() > 0 )
				{
					for( int32 v = 0 ; v < ShapeVertices.Num() ; ++v )
					{
						FVector vtx0 = ShapeVertices[v] + HalfDelta;
						FVector vtx1 = ShapeVertices[(v+1)%ShapeVertices.Num()] + HalfDelta;
						FVector vtx2 = vtx1 - 2.0f * HalfDelta;
						FVector vtx3 = vtx0 - 2.0f * HalfDelta;

						FPoly SidePoly;
						SidePoly.Init();

						SidePoly.Vertices.Add( FVector3f(vtx1 - BaseLocation) );
						SidePoly.Vertices.Add( FVector3f(vtx0 - BaseLocation) );
						SidePoly.Vertices.Add( FVector3f(vtx3 - BaseLocation) );
						SidePoly.Vertices.Add( FVector3f(vtx2 - BaseLocation) );

						if( SidePoly.Finalize( ResultingBrush, 1 ) == 0 )
						{
							new(ResultingBrush->Brush->Polys->Element)FPoly( SidePoly );
						}
					}
				}
			}
			else // not extruding a solid brush
			{
				// Now that we have a set of convex polygons, add them all to the brush.  These will form the top face.
				for( int32  p = 0  ; p < Polygons.Num() ; ++p )
				{
					if( Polygons[p].Finalize( ResultingBrush, 0 ) == 0 )
					{
						new(ResultingBrush->Brush->Polys->Element)FPoly(Polygons[p]);
					}
				}
			}
		}

		// Finish up
		
		ResultingBrush->Brush->BuildBound();

		ResultingBrush->ReregisterAllComponents();

		ShapeVertices.Empty();

		FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

		mode->FinalizeSourceData();
		mode->GetFromSource();

		GEditor->SelectNone( true, true );
		GEditor->SelectActor( ResultingBrush, true, true );

		// Switch back to edit mode
		//FModeTool_GeometryModify* tool = (FModeTool_GeometryModify*)mode->GetCurrentTool();
		//tool->SetCurrentModifier( tool->GetModifier(0) );

		//force a rebuild of the brush (otherwise the auto-update will do it and this will result in the undo buffer being incorrect)


		ABrush::SetNeedRebuild(ResultingBrush->GetLevel());
		
		FBSPOps::RebuildBrush( ResultingBrush->Brush );

		GEditor->RebuildAlteredBSP();

		GEditor->RedrawLevelEditingViewports(true);
	}
}

#if 0
static bool DoesFinalLineIntersectWithShape(const TArray<FVector>& Vertices, const FVector& FinalVertex)
{
	// TODO: improve this, it often fails.

	// Determine if the next line segment would intersect with any of the previous ones in the shape
	for (int32 Index = 0; Index < Vertices.Num() - 1; Index++)
	{
		const FVector& Segment1Start = Vertices[Index];
		const FVector& Segment1End = Vertices[Index + 1];
		const FVector& Segment2Start = Vertices[Vertices.Num() - 1];
		const FVector& Segment2End = FinalVertex;

		// Check that the two line segments are coplanar
		check(FMath::IsNearlyZero(FVector::DotProduct(Segment2Start - Segment1Start, FVector::CrossProduct(Segment1End - Segment1Start, Segment2End - Segment2Start))));

		FVector Segment1Result;
		FVector Segment2Result;
		FMath::SegmentDistToSegmentSafe(Segment1Start, Segment1End, Segment2Start, Segment2End, Segment1Result, Segment2Result);
		if (Segment1Result.Equals(Segment2Result) && !Segment1Result.Equals(Segment1Start) && !Segment1Result.Equals(Segment1End))
		{
			return true;
		}
	}

	return false;
}
#endif

static bool DoLineSegmentsIntersect(const FVector2D& Segment1Start, const FVector2D& Segment1End, const FVector2D& Segment2Start, const FVector2D& Segment2End)
{
	const FVector2D Segment1Dir = Segment1End - Segment1Start;
	const FVector2D Segment2Dir = Segment2End - Segment2Start;

	const double Determinant = FVector2D::CrossProduct(Segment1Dir, Segment2Dir);
	if (!FMath::IsNearlyZero(Determinant))
	{
		const FVector2D SegmentStartDelta = Segment2Start - Segment1Start;
		const double OneOverDet = 1.0 / Determinant;
		const double Seg1Intersection = FVector2D::CrossProduct(SegmentStartDelta, Segment2Dir) * OneOverDet;
		const double Seg2Intersection = FVector2D::CrossProduct(SegmentStartDelta, Segment1Dir) * OneOverDet;

		const double Epsilon = 1/128.0f;
		return (Seg1Intersection > Epsilon && Seg1Intersection < 1.0 - Epsilon && Seg2Intersection > Epsilon && Seg2Intersection < 1.0 - Epsilon);
	}

	return false;
}

/**
* Given an array of points forming an unclosed polygon, determines whether a line segment formed from the final polygon vertex and a given endpoint
* intersects with any edge of the polygon in the 2D plane in which both segments lie.
*
* @param	Vertices		Array of vertices forming unclosed polygon
* @param	EndVertex		Endpoint of line segment starting from final point of polygon
*/
static bool DoesFinalLineIntersectWithShape(const TArray<FVector>& Vertices, const FVector& EndVertex)
{
	// Can't intersect if there are fewer than 2 vertices
	if (Vertices.Num() < 2)
	{
		return false;
	}

	// All line segments in the polygon ought to be coplanar. Hence the problem can be reduced to detecting intersections of line segments
	// projected onto their common plane, using 2D coordinates.

	// Line segment 1 is the line to test against the rest of the shape
	const FVector& Segment1Start = Vertices[Vertices.Num() - 1];
	const FVector& Segment1End = EndVertex;

	const FVector Segment1Dir = Segment1End - Segment1Start;
	const double Segment1Len = Segment1Dir.Size();
	if (FMath::IsNearlyZero(Segment1Len))
	{
		// Treat zero length line segments as non-intersecting
		return false;
	}

	// The direction of segment 1 on the plane will provide the X axis of the 2D basis on the plane
	const FVector ProjectedXAxis = Segment1Dir / Segment1Len;

	for (int32 Index = 0; Index < Vertices.Num() - 1; Index++)
	{
		// Line segment 2 is each edge of the shape
		const FVector& Segment2Start = Vertices[Index];
		const FVector& Segment2End = Vertices[Index + 1];
		const FVector Segment2Dir = Segment2End - Segment2Start;

		const FVector SegmentStartDelta = Segment2Start - Segment1Start;

		// If line segments 1 and 2 are coplanar, the plane normal will be shared, and be calculated from a cross product of their two directions
		FVector PlaneNormal = FVector::CrossProduct(Segment1Dir, Segment2Dir);

		// Check that they are indeed coplanar
		const bool bIsCoplanar = FMath::IsNearlyZero(FVector::DotProduct(SegmentStartDelta, PlaneNormal));
		if (!bIsCoplanar)
		{
			// Non-coplanar line segments can't possibly intersect (ignoring the case of coincident start/end points)
			return false;
		}

		// Parallel line segments will have yielded a zero normal. Attempt to calculate it from the segment start delta.
		// If the lines are coincident, this will also yield a zero normal, resulting in a 1D basis (which is still sufficient to detect segment overlaps in projection space).
		const bool bParallel = (FMath::IsNearlyZero(PlaneNormal.SizeSquared()));
		if (bParallel)
		{
			PlaneNormal = FVector::CrossProduct(Segment1Dir, SegmentStartDelta);
		}

		// Get the Y axis of the 2D basis from the X axis and the plane normal
		const FVector ProjectedYAxis = FVector::CrossProduct(PlaneNormal.GetSafeNormal(), ProjectedXAxis);

		// Project 3d points onto plane
		const FVector2D ProjectedSegment1Start(0.0, 0.0);
		const FVector2D ProjectedSegment1End(Segment1Len, 0.0);
		const FVector2D ProjectedSegment2Start(FVector::DotProduct(ProjectedXAxis, SegmentStartDelta), FVector::DotProduct(ProjectedYAxis, SegmentStartDelta));
		const FVector2D ProjectedSegment2End(FVector::DotProduct(ProjectedXAxis, Segment2End - Segment1Start), FVector::DotProduct(ProjectedYAxis, Segment2End - Segment1Start));

		// Now check intersection of 2d segments.
		if (DoLineSegmentsIntersect(ProjectedSegment1Start, ProjectedSegment1End, ProjectedSegment2Start, ProjectedSegment2End))
		{
			return true;
		}
	}

	return false;
}


/**
* @return		true if the key was handled by this editor mode tool.
*/
bool UGeomModifier_Pen::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	bool bResult = false;
#if WITH_EDITORONLY_DATA
	if( ViewportClient->IsOrtho() && Event == IE_Pressed )
	{
		const bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
		const bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
		const bool bAltDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);

		// CTRL+RightClick (or SPACE bar) adds a vertex to the world

		if( (bCtrlDown && !bShiftDown && !bAltDown && Key == EKeys::RightMouseButton) || Key == EKeys::SpaceBar )
		{
			// if we're trying to edit vertices in a different viewport to the one we started in then popup a warning
			if( ShapeVertices.Num() && ViewportClient != UsingViewportClient )
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "GeomModifierPen_Warning_AddingVertexInWrongViewport", "Vertices can only be added to one viewport at a time." ) );
				return true;
			}
			if( ShapeVertices.Num() && MouseWorldSpacePos.Equals( ShapeVertices[0] ) )
			{
				if (!DoesFinalLineIntersectWithShape(ShapeVertices, ShapeVertices[0]))
				{
					Apply();
					bResult = true;
				}
			}
			else
			{
				if (!DoesFinalLineIntersectWithShape(ShapeVertices, MouseWorldSpacePos))
				{
					UsingViewportClient = ViewportClient;
					ShapeVertices.Add( MouseWorldSpacePos );
					bResult = true;
				}
			}
		}
		else if( Key == EKeys::Escape || Key == EKeys::BackSpace )
		{
			if( ShapeVertices.Num() )
			{
				ShapeVertices.RemoveAt( ShapeVertices.Num() - 1 );
			}

			bResult = true;
		}
		else if( Key == EKeys::Enter )
		{
			if (ShapeVertices.Num() > 0 && !DoesFinalLineIntersectWithShape(ShapeVertices, ShapeVertices[0]))
			{
				Apply();
				bResult = true;
			}
		}
	}

	if( bResult )
	{
		GEditor->RedrawLevelEditingViewports( true );
	}
#endif // WITH_EDITORONLY_DATA

	return bResult;
}

void UGeomModifier_Pen::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	FModeTool_GeometryModify* tool = (FModeTool_GeometryModify*)mode->GetCurrentTool();
	if( tool->GetCurrentModifier() != this )
	{
		return;
	}

	// Only draw in ortho viewports

	if( !((FEditorViewportClient*)(Viewport->GetClient()))->IsOrtho() )
	{
		return;
	}

	FLinearColor Color = bCreateBrushShape ? GEngine->C_BrushShape : GEngine->C_BrushWire;

	// If we have more than 2 vertices placed, connect them with lines

	if( ShapeVertices.Num() > 1 )
	{
		for( int32 v = 0 ; v < ShapeVertices.Num() - 1 ; ++v )
		{
			PDI->DrawLine( ShapeVertices[v], ShapeVertices[ v+1 ], Color, SDPG_Foreground );
		}
	}

	// Draw vertices for each point the user has put down

	for( int32 v = 0 ; v < ShapeVertices.Num() ; ++v )
	{
		PDI->DrawPoint( ShapeVertices[v], Color, 6.f, SDPG_Foreground );
	}

	if( ShapeVertices.Num() )
	{
		if (!DoesFinalLineIntersectWithShape(ShapeVertices, MouseWorldSpacePos))
		{
			// Draw a dashed line from the last placed vertex to the current mouse position
			DrawDashedLine(PDI, ShapeVertices[ShapeVertices.Num() - 1], MouseWorldSpacePos, FLinearColor(1, 0.5f, 0), GEditor->GetGridSize(), SDPG_Foreground);
		}
	}

	if( ShapeVertices.Num() > 2 )
	{
		if (!DoesFinalLineIntersectWithShape(ShapeVertices, ShapeVertices[0]))
		{
			// Draw a darkened dashed line to show what the completed shape will look like
			DrawDashedLine(PDI, ShapeVertices[ShapeVertices.Num() - 1], ShapeVertices[0], FLinearColor(.5, 0, 0), GEditor->GetGridSize(), SDPG_Foreground);
		}
	}

	// Draw a box where the next vertex will be placed

	int32 BoxSz = static_cast<int32>( FMath::Max( GEditor->GetGridSize() / 2, 1.f ) );
	DrawWireBox(PDI, FBox::BuildAABB( MouseWorldSpacePos, FVector(BoxSz,BoxSz,BoxSz) ), FLinearColor(1,1,1), SDPG_Foreground);
}

void UGeomModifier_Pen::DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas)
{
}

void UGeomModifier_Pen::Tick(FEditorViewportClient* ViewportClient,float DeltaTime)
{
	if( GCurrentLevelEditingViewportClient == ViewportClient )
	{
		FVector NewMouseWorldSpacePos = ComputeWorldSpaceMousePos(ViewportClient);
		// If the grid is enabled, figure out where the nearest grid location is to the mouse cursor
		if( GetDefault<ULevelEditorViewportSettings>()->GridEnabled )
		{
			NewMouseWorldSpacePos = NewMouseWorldSpacePos.GridSnap( GEditor->GetGridSize() );
		}

		// If the mouse position has moved, update the viewport
		if( NewMouseWorldSpacePos != MouseWorldSpacePos )
		{
			MouseWorldSpacePos = NewMouseWorldSpacePos;
			GEditor->RedrawLevelEditingViewports( true );
		}
	}
}

/*------------------------------------------------------------------------------
	UGeomModifier_Clip
------------------------------------------------------------------------------*/

namespace GeometryClipping {

/**
* Creates a giant brush aligned with this plane.
*
* @param	OutGiantBrush		[out] The new brush.
* @param	InPlane				Plane with which to align the brush.
*
* NOTE: it is up to the caller to set up the new brush upon return in regards to it's CSG operation and flags.
*/
static void BuildGiantAlignedBrush( ABrush& OutGiantBrush, const FPlane& InPlane )
{
	OutGiantBrush.SetActorLocation(FVector::ZeroVector, false);
	OutGiantBrush.SetPivotOffset(FVector::ZeroVector);

	verify( OutGiantBrush.Brush );
	verify( OutGiantBrush.Brush->Polys );

	OutGiantBrush.Brush->Polys->Element.Empty();

	// Create a list of vertices that can be used for the new brush
	FVector3f vtxs[8];

	FPlane FlippedPlane = InPlane.Flip();
	FPoly TempPoly = FPoly::BuildInfiniteFPoly( FlippedPlane );
	TempPoly.Finalize(&OutGiantBrush,0);
	vtxs[0] = TempPoly.Vertices[0];
	vtxs[1] = TempPoly.Vertices[1];
	vtxs[2] = TempPoly.Vertices[2];
	vtxs[3] = TempPoly.Vertices[3];

	FlippedPlane = FlippedPlane.Flip();
	FPoly TempPoly2 = FPoly::BuildInfiniteFPoly( FlippedPlane );
	vtxs[4] = FVector3f(TempPoly2.Vertices[0] + (TempPoly2.Normal * -(UE_OLD_WORLD_MAX)));	vtxs[5] = FVector3f(TempPoly2.Vertices[1] + (TempPoly2.Normal * -(UE_OLD_WORLD_MAX)));
	vtxs[6] = FVector3f(TempPoly2.Vertices[2] + (TempPoly2.Normal * -(UE_OLD_WORLD_MAX)));	vtxs[7] = FVector3f(TempPoly2.Vertices[3] + (TempPoly2.Normal * -(UE_OLD_WORLD_MAX)));

	// Create the polys for the new brush.
	FPoly newPoly;

	// TOP
	newPoly.Init();
	newPoly.Base = vtxs[0];
	newPoly.Vertices.Add( vtxs[0] );
	newPoly.Vertices.Add( vtxs[1] );
	newPoly.Vertices.Add( vtxs[2] );
	newPoly.Vertices.Add( vtxs[3] );
	newPoly.Finalize(&OutGiantBrush,0);
	new(OutGiantBrush.Brush->Polys->Element)FPoly(newPoly);

	// BOTTOM
	newPoly.Init();
	newPoly.Base = vtxs[4];
	newPoly.Vertices.Add( vtxs[4] );
	newPoly.Vertices.Add( vtxs[5] );
	newPoly.Vertices.Add( vtxs[6] );
	newPoly.Vertices.Add( vtxs[7] );
	newPoly.Finalize(&OutGiantBrush,0);
	new(OutGiantBrush.Brush->Polys->Element)FPoly(newPoly);

	// SIDES
	// 1
	newPoly.Init();
	newPoly.Base = vtxs[1];
	newPoly.Vertices.Add( vtxs[1] );
	newPoly.Vertices.Add( vtxs[0] );
	newPoly.Vertices.Add( vtxs[7] );
	newPoly.Vertices.Add( vtxs[6] );
	newPoly.Finalize(&OutGiantBrush,0);
	new(OutGiantBrush.Brush->Polys->Element)FPoly(newPoly);

	// 2
	newPoly.Init();
	newPoly.Base = vtxs[2];
	newPoly.Vertices.Add( vtxs[2] );
	newPoly.Vertices.Add( vtxs[1] );
	newPoly.Vertices.Add( vtxs[6] );
	newPoly.Vertices.Add( vtxs[5] );
	newPoly.Finalize(&OutGiantBrush,0);
	new(OutGiantBrush.Brush->Polys->Element)FPoly(newPoly);

	// 3
	newPoly.Init();
	newPoly.Base = vtxs[3];
	newPoly.Vertices.Add( vtxs[3] );
	newPoly.Vertices.Add( vtxs[2] );
	newPoly.Vertices.Add( vtxs[5] );
	newPoly.Vertices.Add( vtxs[4] );
	newPoly.Finalize(&OutGiantBrush,0);
	new(OutGiantBrush.Brush->Polys->Element)FPoly(newPoly);

	// 4
	newPoly.Init();
	newPoly.Base = vtxs[0];
	newPoly.Vertices.Add( vtxs[0] );
	newPoly.Vertices.Add( vtxs[3] );
	newPoly.Vertices.Add( vtxs[4] );
	newPoly.Vertices.Add( vtxs[7] );
	newPoly.Finalize(&OutGiantBrush,0);
	new(OutGiantBrush.Brush->Polys->Element)FPoly(newPoly);

	// Finish creating the new brush.
	OutGiantBrush.Brush->BuildBound();
}

/**
 * Clips the specified brush against the specified plane.
 *
 * @param		InWorld		World context
 * @param		InPlane		The plane to clip against.
 * @param		InBrush		The brush to clip.
 * @return					The newly created brush representing the portion of the brush in the plane's positive halfspace.
 */
static ABrush* ClipBrushAgainstPlane( const FPlane& InPlane, ABrush* InBrush)
{
	UWorld* World = InBrush->GetWorld();
	ULevel* BrushLevel = InBrush->GetLevel();
	
	// Create a giant brush in the level of the source brush to use in the intersection process.
	ABrush* ClippedBrush = NULL;

	// When clipping non-builder brushes, create a duplicate of the brush 
	// to clip. This duplicate will replace the existing brush. 
	if( !FActorEditorUtils::IsABuilderBrush(InBrush) )
	{

		// Select only the original brush to prevent other actors from being duplicated. 
		GEditor->SelectNone( false, true );
		GEditor->SelectActor( InBrush, true, false, false );

		// Duplicate the original brush. This will serve as our clipped brush. 
		GEditor->edactDuplicateSelected( BrushLevel, false );

		// Clipped brush should be the only selected 
		// actor if the duplication didn't fail. 
		ClippedBrush = GEditor->GetSelectedActors()->GetTop<ABrush>();
	}
	// To clip the builder brush, instead of replacing it, spawn a 
	// temporary brush to clip. Then, copy that to the builder brush.
	else
	{
		// NOTE: This brush is discarded later on after copying the values to the builder brush. 
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.OverrideLevel = BrushLevel;
		SpawnInfo.Template = InBrush;
		ClippedBrush = World->SpawnActor<ABrush>( InBrush->GetClass(), SpawnInfo );
		check( ClippedBrush );
	}

	// It's possible that the duplication failed. 
	if( !ClippedBrush )
	{
		return NULL;
	}

	// The brushes should have the same class otherwise 
	// perhaps there were additional brushes were selected. 
	check( ClippedBrush->GetClass() == InBrush->GetClass() );

	ClippedBrush->Brush = NewObject<UModel>(ClippedBrush, NAME_None, RF_Transactional);
	ClippedBrush->Brush->Initialize(nullptr);
	ClippedBrush->GetBrushComponent()->Brush = ClippedBrush->Brush;

	GeometryClipping::BuildGiantAlignedBrush( *ClippedBrush, InPlane );

	ClippedBrush->BrushType = InBrush->BrushType;
	ClippedBrush->SetFlags( InBrush->GetFlags() );
	ClippedBrush->PolyFlags = InBrush->PolyFlags;

	// Create a BSP for the brush that is being clipped.
	FBSPOps::bspBuild( InBrush->Brush, FBSPOps::BSP_Optimal, 15, 70, 1, 0 );
	FBSPOps::bspRefresh( InBrush->Brush, true );
	FBSPOps::bspBuildBounds( InBrush->Brush );

	// Intersect the giant brush with the source brush's BSP.  This will give us the finished, clipping brush
	// contained inside of the giant brush.

	ClippedBrush->Modify(false);
	InBrush->Brush->Modify(false);
	GEditor->bspBrushCSG( ClippedBrush, InBrush->Brush, 0, Brush_MAX, CSG_Intersect, false, false, true );
	FBSPOps::bspUnlinkPolys( ClippedBrush->Brush );

	// Remove all polygons on the giant brush that don't match the normal of the clipping plane

	for( int32 p = 0 ; p < ClippedBrush->Brush->Polys->Element.Num() ; ++p )
	{
		FPoly* P = &ClippedBrush->Brush->Polys->Element[p];

		if( P->Finalize( ClippedBrush, 1 ) == 0 )
		{
			if( !FPlane( (FVector)P->Vertices[0], (FVector)P->Normal ).Equals( InPlane, 0.01f ) )
			{
				ClippedBrush->Brush->Polys->Element.RemoveAt( p );
				p = -1;
			}
		}
	}

	// The BSP "CSG_Intersect" code sometimes creates some nasty polygon fragments so clean those up here before going further.

	FPoly::OptimizeIntoConvexPolys( ClippedBrush, ClippedBrush->Brush->Polys->Element );

	// Clip each polygon in the original brush against the clipping plane.  For every polygon that is behind the plane or split by it, keep the back portion.

	FVector PlaneBase = FVector( InPlane.X, InPlane.Y, InPlane.Z ) * InPlane.W;

	for( int32 p = 0 ; p < InBrush->Brush->Polys->Element.Num() ; ++p )
	{
		FPoly Poly = InBrush->Brush->Polys->Element[p];

		FPoly front, back;

		int32 res = Poly.SplitWithPlane( (FVector3f)PlaneBase, (FVector3f)InPlane.GetSafeNormal(), &front, &back, true );

		switch( res )
		{
			case SP_Back:
				ClippedBrush->Brush->Polys->Element.Add( Poly );
				break;

			case SP_Split:
				ClippedBrush->Brush->Polys->Element.Add( back );
				break;
		}
	}

	// At this point we have a clipped brush with optimized capping polygons so we can finish up by fixing it's ordering in the actor array and other misc things.

	ClippedBrush->CopyPosRotScaleFrom( InBrush );
	ClippedBrush->PolyFlags = InBrush->PolyFlags;

	// Clean the brush up.
	for( int32 poly = 0 ; poly < ClippedBrush->Brush->Polys->Element.Num() ; poly++ )
	{
		FPoly* Poly = &(ClippedBrush->Brush->Polys->Element[poly]);
		Poly->iLink = poly;
		Poly->Normal = FVector3f::ZeroVector;
		Poly->Finalize(ClippedBrush,0);
	}

	// One final pass to clean the polyflags of all temporary settings.
	for( int32 poly = 0 ; poly < ClippedBrush->Brush->Polys->Element.Num() ; poly++ )
	{
		FPoly* Poly = &(ClippedBrush->Brush->Polys->Element[poly]);
		Poly->PolyFlags &= ~PF_EdCut;
		Poly->PolyFlags &= ~PF_EdProcessed;
	}

	// Move the new brush to where the new brush was to preserve brush ordering.
	ABrush* BuilderBrush = World->GetDefaultBrush();
	if( InBrush == BuilderBrush )
	{
		// Special-case behavior for the builder brush.

		// Copy the temporary brush back over onto the builder brush (keeping object flags)
		BuilderBrush->Modify(false);
		FBSPOps::csgCopyBrush( BuilderBrush, ClippedBrush, BuilderBrush->PolyFlags, BuilderBrush->GetFlags(), 0, true );
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		Layers->DisassociateActorFromLayers( ClippedBrush );
		World->EditorDestroyActor( ClippedBrush, false );
		// Note that we're purposefully returning non-NULL here to report that the clip was successful,
		// even though the ClippedBrush has been destroyed!
	}
	else
	{
		// Remove the old brush.
		const int32 ClippedBrushIndex = BrushLevel->Actors.Num() - 1;
		check( BrushLevel->Actors[ClippedBrushIndex] == ClippedBrush );
		BrushLevel->Actors.RemoveAt(ClippedBrushIndex);

		// Add the new brush right after the old brush.
		const int32 OldBrushIndex = BrushLevel->Actors.Find( InBrush );
		check( OldBrushIndex != INDEX_NONE );
		BrushLevel->Actors.Insert( ClippedBrush, OldBrushIndex+1 );
	}

	return ClippedBrush;
}

} // namespace GeometryClipping
UGeomModifier_Clip::UGeomModifier_Clip(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Clip", "Clip");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Clip", "Tooltip", "Given a dividing plane, cut the geometry into two pieces, optionally discarding one of them. This operation only works in an orthographic viewport.  Define the vertices of the dividing plane with the space bar, and press Enter to apply.");
	bFlipNormal = false;
	bSplit = false;
	ToolbarIconName = TEXT("GeometryMode.ClipTool");
}

void UGeomModifier_Clip::WasActivated()
{
	ClipMarkers.Empty();
}

bool UGeomModifier_Clip::SupportsCurrentSelection()
{
	if ( !Super::SupportsCurrentSelection() )
	{
		return false;
	}

	FEdModeGeometry* Mode = (FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && (Mode->GetSelectionState() ? false : true);
}

bool UGeomModifier_Clip::OnApply()
{
	ApplyClip( bSplit, bFlipNormal );

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	bPendingPivotOffsetUpdate = true;
	return true;
}

void UGeomModifier_Clip::ApplyClip( bool InSplit, bool InFlipNormal )
{
	if ( !GLastKeyLevelEditingViewportClient )
	{
		return;
	}

	// Assemble the set of selected brushes.
	TArray<ABrush*> Brushes;
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		ABrush* Brush = Cast< ABrush >( Actor );
		if( Brush )
		{
			Brushes.Add( Brush );
		}
	}

	// Do nothing if no brushes are selected.
	if ( Brushes.Num() == 0 )
	{
		return;
	}

	// Make sure enough clip markers have been placed.
	if( ClipMarkers.Num() != 2 )
	{
		GeomError( NSLOCTEXT("UnrealEd", "Error_NotEnoughClipMarkers", "You haven't placed enough clip markers to perform this operation.").ToString() );
		return;
	}

	// Focus has to be in an orthographic viewport so the editor can determine where the third point on the plane is
	if( !GLastKeyLevelEditingViewportClient->IsOrtho() )
	{
		GeomError( NSLOCTEXT("UnrealEd", "Error_BrushClipViewportNotOrthographic", "The focus needs to be in an orthographic viewport for brush clipping to work.").ToString() );
		return;
	}


	// Create a clipping plane based on ClipMarkers present in the level.
	const FVector vtx1 = ClipMarkers[0];
	const FVector vtx2 = ClipMarkers[1];
	FVector vtx3;

	// Compute the third vertex based on the viewport orientation.

	vtx3 = vtx1;

	switch( GLastKeyLevelEditingViewportClient->ViewportType )
	{
		case LVT_OrthoXY:
			vtx3.Z -= 64;
			break;

		case LVT_OrthoXZ:
			vtx3.Y -= 64;
			break;

		case LVT_OrthoYZ:
			vtx3.X -= 64;
			break;
	}

	// Perform the clip.
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushClip", "Brush Clip") );

		GEditor->SelectNone( false, true );

		// Clip the brush list.
		TArray<ABrush*> NewBrushes;
		TArray<ABrush*> OldBrushes;

		for ( int32 BrushIndex = 0 ; BrushIndex < Brushes.Num() ; ++BrushIndex )
		{
			ABrush* SrcBrush = Brushes[ BrushIndex ];

			// Compute a clipping plane in the local frame of the brush.
			const FTransform ToBrushWorld( SrcBrush->ActorToWorld() );
			const FVector LocalVtx1( ToBrushWorld.InverseTransformPosition( vtx1 ) );
			const FVector LocalVtx2( ToBrushWorld.InverseTransformPosition( vtx2 ) );
			const FVector LocalVtx3( ToBrushWorld.InverseTransformPosition( vtx3 ) );

			FVector PlaneNormal( (LocalVtx2 - LocalVtx1) ^ (LocalVtx3 - LocalVtx1) );
			if( PlaneNormal.SizeSquared() < THRESH_ZERO_NORM_SQUARED )
			{
				GeomError( NSLOCTEXT("UnrealEd", "Error_ClipUnableToComputeNormal", "Unable to compute normal for brush clip!").ToString() );
				continue;
			}
			PlaneNormal.Normalize();

			FPlane ClippingPlane( LocalVtx1, PlaneNormal );
			if ( InFlipNormal )
			{
				ClippingPlane = ClippingPlane.Flip();
			}
			
			// Is the brush a builder brush?
			const bool bIsBuilderBrush = FActorEditorUtils::IsABuilderBrush(SrcBrush);

			// Perform the clip.
			bool bCreatedBrush = false;
			ABrush* NewBrush = GeometryClipping::ClipBrushAgainstPlane( ClippingPlane, SrcBrush );
			if ( NewBrush )
			{
				// Select the src brush for builders, or the returned brush for non-builders.
				if ( !bIsBuilderBrush )
				{
					NewBrushes.Add( NewBrush );
				}
				else
				{
					NewBrushes.Add( SrcBrush );
				}
				bCreatedBrush = true;
			}

			// If we're doing a split instead of just a plain clip . . .
			if( InSplit )
			{
				// Don't perform a second clip if the builder brush was already split.
				if ( !bIsBuilderBrush || !bCreatedBrush )
				{
					// Clip the brush against the flipped clipping plane.
					ABrush* NewBrush2 = GeometryClipping::ClipBrushAgainstPlane( ClippingPlane.Flip(), SrcBrush );
					if ( NewBrush2 )
					{
						// We don't add the brush to the list of new brushes, so that only new brushes
						// in the non-cleaved halfspace of the clipping plane will be selected.
						bCreatedBrush = true;
					}
				}
			}

			// Destroy source brushes that aren't builders.
			if ( !bIsBuilderBrush )
			{
				OldBrushes.Add( SrcBrush );
			}
		}

		// Clear selection to prevent the second clipped brush from being selected. 
		// When both are selected, it's hard to tell that the brush is clipped.
		GEditor->SelectNone( false, true );

		// Delete old brushes.
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		for ( int32 BrushIndex = 0 ; BrushIndex < OldBrushes.Num() ; ++BrushIndex )
		{
			ABrush* OldBrush = OldBrushes[ BrushIndex ];
			Layers->DisassociateActorFromLayers( OldBrush );
			OldBrush->GetWorld()->EditorDestroyActor( OldBrush, true );
		}

		// Select new brushes.
		for ( int32 BrushIndex = 0 ; BrushIndex < NewBrushes.Num() ; ++BrushIndex )
		{
			ABrush* NewBrush = NewBrushes[ BrushIndex ];
			GEditor->SelectActor( NewBrush, true, false );
		}

		// Notify editor of new selection state.
		GEditor->NoteSelectionChange();
	}

	FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	Mode->FinalizeSourceData();
	Mode->GetFromSource();
}

bool UGeomModifier_Clip::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	bool bResult = false;

	if( ViewportClient->IsOrtho() && Event == IE_Pressed )
	{
		const bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
		const bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
		const bool bAltDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);

		if( (bCtrlDown && !bShiftDown && !bAltDown && Key == EKeys::RightMouseButton) || Key == EKeys::SpaceBar )
		{
			// if the user has 2 markers placed and the click location is on top of the second point, perform the cllck.  This is a shortcut the LDs wanted.

			if( ClipMarkers.Num() == 2 )
			{
				const FVector* Pos = &ClipMarkers[1];

				if( Pos->Equals( SnappedMouseWorldSpacePos ) )
				{
					OnApply();
					return true;
				}
			}

			// If there are already 2 clip markers in the world, clear them out.

			if( ClipMarkers.Num() > 1 )
			{
				ClipMarkers.Empty();
			}

			ClipMarkers.Add( SnappedMouseWorldSpacePos );
			bResult = true;
		}
		else if( Key == EKeys::Escape || Key == EKeys::BackSpace )
		{
			if( ClipMarkers.Num() )
			{
				ClipMarkers.RemoveAt( ClipMarkers.Num() - 1 );
			}

			bResult = true;
		}
		else if( Key == EKeys::Enter )
		{
			// If the user has 1 marker placed when they press ENTER, go ahead and place a second one at the current mouse location.
			// This allows LDs to place one point, move to a good spot and press ENTER for a quick clip.

			if( ClipMarkers.Num() == 1 )
			{
				ClipMarkers.Add( SnappedMouseWorldSpacePos );
			}

			ApplyClip( bAltDown, bShiftDown );

			bResult = true;
		}
	}

	if( bResult )
	{
		GEditor->RedrawLevelEditingViewports( true );
	}

	return bResult;
}

void UGeomModifier_Clip::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	FModeTool_GeometryModify* tool = (FModeTool_GeometryModify*)mode->GetCurrentTool();
	if( tool->GetCurrentModifier() != this )
	{
		return;
	}

	// Only draw in ortho viewports

	if( !((FEditorViewportClient*)(Viewport->GetClient()))->IsOrtho() )
	{
		return;
	}

	// Draw a yellow box on each clip marker

	for( int32 x = 0 ; x < ClipMarkers.Num() ; ++x )
	{
		FVector* vtx = &ClipMarkers[x];

		PDI->DrawPoint( *vtx, FLinearColor(1,0,0), 6.f, SDPG_Foreground );
	}

	// If 2 markers are placed, draw a line connecting them and a line showing the clip normal.
	// If 1 marker is placed, draw a dashed line and normal to show where the clip plane will appear if the user commits.

	if( ClipMarkers.Num() )
	{
		FVector LineStart = ClipMarkers[0];
		FVector LineEnd = (ClipMarkers.Num() == 2) ? ClipMarkers[1] : SnappedMouseWorldSpacePos;

		if( ClipMarkers.Num() == 1 )
		{
			DrawDashedLine( PDI, LineStart, LineEnd, FLinearColor(1,.5,0), GEditor->GetGridSize(), SDPG_Foreground );
		}
		else
		{
			PDI->DrawLine( LineStart, LineEnd, FLinearColor(1,0,0), SDPG_Foreground );
		}

		FVector vtx1, vtx2, vtx3;
		FPoly NormalPoly;

		vtx1 = LineStart;
		vtx2 = LineEnd;

		vtx3 = vtx1;

		const FEditorViewportClient* ViewportClient = static_cast<FEditorViewportClient*>( Viewport->GetClient() );
		switch( ViewportClient->ViewportType )
		{
			case LVT_OrthoXY:
				vtx3.Z -= 64;
				break;
			case LVT_OrthoXZ:
				vtx3.Y -= 64;
				break;
			case LVT_OrthoYZ:
				vtx3.X -= 64;
				break;
		}

		NormalPoly.Vertices.Add( (FVector3f)vtx1 );
		NormalPoly.Vertices.Add( (FVector3f)vtx2 );
		NormalPoly.Vertices.Add( (FVector3f)vtx3 );

		if( !NormalPoly.CalcNormal(1) )
		{
			FVector Start = ( vtx1 + vtx2 ) / 2.;

			double NormalLength = (vtx2 - vtx1).Size() / 2.;

			if( ClipMarkers.Num() == 1 )
			{
				DrawDashedLine( PDI, Start, Start + (FVector)NormalPoly.Normal * NormalLength, FLinearColor(1,.5,0), GEditor->GetGridSize(), SDPG_Foreground );
			}
			else
			{
				PDI->DrawLine( Start, Start + (FVector)NormalPoly.Normal * NormalLength, FLinearColor(1,0,0), SDPG_Foreground );
			}
		}
	}

	// Draw a box at the cursor location

	int32 BoxSz = static_cast<int32>( FMath::Max( GEditor->GetGridSize() / 2, 1.f ) );
	DrawWireBox(PDI, FBox::BuildAABB( SnappedMouseWorldSpacePos, FVector(BoxSz,BoxSz,BoxSz) ), FLinearColor(1,1,1), SDPG_Foreground);
}

void UGeomModifier_Clip::DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas)
{
}

void UGeomModifier_Clip::Tick(FEditorViewportClient* ViewportClient,float DeltaTime)
{
	if( GCurrentLevelEditingViewportClient == ViewportClient )
	{
		// Figure out where the nearest grid location is to the mouse cursor
		FVector NewSnappedMouseWorldSpacePos = ComputeWorldSpaceMousePos(ViewportClient).GridSnap( GEditor->GetGridSize() );

		// If the snapped mouse position has moved, update the viewport
		if( NewSnappedMouseWorldSpacePos != SnappedMouseWorldSpacePos )
		{
			SnappedMouseWorldSpacePos = NewSnappedMouseWorldSpacePos;
			GEditor->RedrawLevelEditingViewports( true );
		}
	}
}


UGeomModifier_Delete::UGeomModifier_Delete(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Delete", "Delete");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Delete", "Tooltip", "Deletes the selected geometry elements (vertices, edges or polygons).");
	bPushButton = true;
	bAppearsInToolbar = false;
}


bool UGeomModifier_Delete::SupportsCurrentSelection()
{
	FEdModeGeometry* Mode = (FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && (Mode->HavePolygonsSelected() || Mode->HaveVerticesSelected());
}


bool UGeomModifier_Delete::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	bool bHandled = false;

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		// Polys

		for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
		{
			FGeomPoly* gp = &go->PolyPool[p];

			if( gp->IsSelected() )
			{
				gp->GetParentObject()->GetActualBrush()->Brush->Polys->Element[ gp->ActualPolyIndex ].PolyFlags |= PF_GeomMarked;
				bHandled = 1;
			}
		}

		for( int32 p = 0 ; p < go->GetActualBrush()->Brush->Polys->Element.Num() ; ++p )
		{
			if( (go->GetActualBrush()->Brush->Polys->Element[ p ].PolyFlags&PF_GeomMarked) > 0 )
			{
				go->GetActualBrush()->Brush->Polys->Element.RemoveAt( p );
				p = -1;
			}
		}

		// Verts

		for( int32 v = 0 ; v < go->VertexPool.Num() ; ++v )
		{
			FGeomVertex* gv = &go->VertexPool[v];

			if( gv->IsSelected() )
			{
				for( int32 x = 0 ; x < gv->GetParentObject()->GetActualBrush()->Brush->Polys->Element.Num() ; ++x )
				{
					FPoly* Poly = &gv->GetParentObject()->GetActualBrush()->Brush->Polys->Element[x];
					Poly->RemoveVertex( (FVector)*gv );
					bHandled = 1;
				}
			}
		}

		go->GetActualBrush()->SavedSelections.Empty();
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	// Reset the pivot point to the newest selected object.
	AActor* SelectedActor = Cast<AActor>(GEditor->GetSelectedActors()->GetBottom(AActor::StaticClass()));

	GEditor->GetSelectedActors()->Modify();

	if(SelectedActor)
	{
		FEditorModeTools& Tools = GLevelEditorModeTools();
		Tools.SetPivotLocation( SelectedActor->GetActorLocation() , false );
	}
	
	bPendingPivotOffsetUpdate = true;
	return bHandled;
}

UGeomModifier_Create::UGeomModifier_Create(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Create", "Create");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Create", "Tooltip", "Creates a new polygon from the selected vertices. The vertices must be selected in clockwise order to create a poly with an outward facing normal.");
	bPushButton = true;
}

bool UGeomModifier_Create::SupportsCurrentSelection()
{
	FEdModeGeometry* Mode = (FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && Mode->HaveVerticesSelected();
}

bool UGeomModifier_Create::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		go->CompileSelectionOrder();

		// Create an ordered list of vertices based on the selection order.

		TArray<FGeomVertex*> Verts;
		for( int32 x = 0 ; x < go->SelectionOrder.Num() ; ++x )
		{
			FGeomBase* obj = go->SelectionOrder[x];
			if( obj->IsVertex() )
			{
				Verts.Add( (FGeomVertex*)obj );
			}
		}

		if( Verts.Num() > 2 )
		{
			// Create new geometry based on the selected vertices

			FPoly* NewPoly = new( go->GetActualBrush()->Brush->Polys->Element )FPoly();

			NewPoly->Init();

			for( int32 x = 0 ; x < Verts.Num() ; ++x )
			{
				FGeomVertex* gv = Verts[x];

				new(NewPoly->Vertices) FVector3f(*gv);
			}

			NewPoly->Normal = FVector3f::ZeroVector;
			NewPoly->Base = *Verts[0];
			NewPoly->PolyFlags = PF_DefaultFlags;
		}
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	bPendingPivotOffsetUpdate = true;
	return true;
}

UGeomModifier_Flip::UGeomModifier_Flip(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Flip", "Flip");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Flip", "Tooltip", "Flips the normal of the selected polygon so that it faces the other way.");
	bPushButton = true;
	ToolbarIconName = TEXT("GeometryMode.FlipTool");
}

bool UGeomModifier_Flip::SupportsCurrentSelection()
{
	if ( !Super::SupportsCurrentSelection() )
	{
		return false;
	}

	// Supports polygons selected and objects selected
	FEdModeGeometry* Mode = (FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && (!Mode->HaveEdgesSelected() && !Mode->HaveVerticesSelected());
}

bool UGeomModifier_Flip::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	bool bHavePolygonsSelected = mode->HavePolygonsSelected();

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
		{
			FGeomPoly* gp = &go->PolyPool[p];

			if( gp->IsSelected() || !bHavePolygonsSelected )
			{
				FPoly* Poly = &go->GetActualBrush()->Brush->Polys->Element[ gp->ActualPolyIndex ];
				Poly->Reverse();
			}
		}
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	bPendingPivotOffsetUpdate = true;
	return true;
}

UGeomModifier_Split::UGeomModifier_Split(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Split", "Split");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Pen", "Split_Tooltip", "Create blocking volumes or freeform geometry by creating a series of anchor points.  Note, you must be in a 2D viewport with focus to use the pen tool.  Create at least 3 anchor points in an orthographic viewport using the spacebar to commit an anchor point.  Complete the shape by creating an anchor on top of the first one.  Use bAutoExtrude with ExtrudeDepth to create a 3D shape.");
	bPushButton = true;
	ToolbarIconName = TEXT("GeometryMode.SplitTool");
}

bool UGeomModifier_Split::SupportsCurrentSelection()
{
	FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	// This modifier assumes that a single geometry object is selected
	if (Mode == nullptr || Mode->CountObjectsSelected() != 1 )
	{
		return false;
	}

	int32 NumPolygonsSelected = Mode->CountSelectedPolygons();
	int32 NumEdgesSelected = Mode->CountSelectedEdges();
	int32 NumVerticesSelected = Mode->CountSelectedVertices();

	if ((NumPolygonsSelected == 1 && NumEdgesSelected == 1 && NumVerticesSelected == 0)		// Splitting a face at an edge mid point (scalpel)
		|| (NumPolygonsSelected == 0 && NumEdgesSelected > 0 && NumVerticesSelected == 0)	// Splitting a brush at an edge mid point (ring cut)
		|| (NumPolygonsSelected == 1 && NumEdgesSelected == 0 && NumVerticesSelected == 2)	// Splitting a polygon across 2 vertices
		|| (NumPolygonsSelected == 0 && NumEdgesSelected == 0 && NumVerticesSelected == 2)	// Splitting a brush across 2 vertices
		)
	{
		return true;
	}

	return false;
}

bool UGeomModifier_Split::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	// Get a pointer to the selected geom object

	FGeomObjectPtr GeomObject = NULL;
	if (FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor)
	{
		GeomObject = *Itor;
	}

	if( !GeomObject.IsValid() )
	{
		return false;
	}

	// Count up how many of each subobject are selected so we can determine what the user is trying to split

	int32 NumPolygonsSelected = mode->CountSelectedPolygons();
	int32 NumEdgesSelected = mode->CountSelectedEdges();
	int32 NumVerticesSelected = mode->CountSelectedVertices();

	if( NumPolygonsSelected == 1 && NumEdgesSelected == 1 && NumVerticesSelected == 0 )
	{
		//
		// Splitting a face at an edge mid point (scalpel)
		//

		// Get the selected edge
		TArray<FGeomEdge*> Edges;
		mode->GetSelectedEdges( Edges );
		check( Edges.Num() == 1 );

		FGeomEdge* SelectedEdge = Edges[0];

		// Figure out the verts that are part of that edge

		FGeomVertex* Vertex0 = &GeomObject->VertexPool[ SelectedEdge->VertexIndices[0] ];
		FGeomVertex* Vertex1 = &GeomObject->VertexPool[ SelectedEdge->VertexIndices[1] ];

		const FVector3f Vtx0 = *Vertex0->GetActualVertex( Vertex0->ActualVertexIndices[0] );
		const FVector3f Vtx1 = *Vertex1->GetActualVertex( Vertex1->ActualVertexIndices[0] );

		// Get the selected polygon
		TArray<FGeomPoly*> Polygons;
		mode->GetSelectedPolygons( Polygons );
		check( Polygons.Num() == 1 );

		FGeomPoly* Polygon = Polygons[0];
		FPoly* SelectedPoly = Polygon->GetActualPoly();

		// Get the selected brush
		ABrush* Brush = GeomObject->GetActualBrush();

		//
		// Sanity checking
		//
		{
			// 1. Make sure that the selected edge is part of the selected polygon

			if( !SelectedPoly->Vertices.Contains( Vtx0 ) || !SelectedPoly->Vertices.Contains( Vtx1 ) )
			{
				GeomError( NSLOCTEXT("UnrealEd", "Error_SelectedEdgeMustBelongToSelectedPoly", "The edge used for splitting must be part of the selected polygon.").ToString() );
				return false;
			}
		}

		// Generate a base and a normal for the cutting plane

		const FVector3f PlaneNormal( (Vtx1 - Vtx0).GetSafeNormal() );
		const FVector3f PlaneBase = 0.5f*(Vtx1 + Vtx0);

		// Clip the selected polygon against the cutting plane

		FPoly Front, Back;
		Front.Init();
		Back.Init();

		int32 Res = SelectedPoly->SplitWithPlane( PlaneBase, PlaneNormal, &Front, &Back, 1 );

		if( Res == SP_Split )
		{
			TArray<FPoly> NewPolygons;

			NewPolygons.Add( Front );
			NewPolygons.Add( Back );

			// At this point, see if any other polygons in the brush need to have a vertex added to an edge

			FPlane4f CuttingPlane( PlaneBase, PlaneNormal );

			for( int32 p = 0 ; p < Brush->Brush->Polys->Element.Num() ; ++p )
			{
				FPoly* P = &Brush->Brush->Polys->Element[p];

				if( P != SelectedPoly )
				{
					for( int32 v = 0 ; v < P->Vertices.Num() ; ++v )
					{
						FVector3f* v0 = &P->Vertices[v];
						FVector3f* v1 = &P->Vertices[ (v + 1) % P->Vertices.Num() ];

						// Make sure the line formed by the edge actually crosses the plane before checking for the intersection point.

						if( ( CuttingPlane.PlaneDot( *v0 ) < 0.0f ) != ( CuttingPlane.PlaneDot( *v1 ) < 0.0f ) )
						{
							FVector3f Intersection = FMath::LinePlaneIntersection( *v0, *v1, CuttingPlane );

							// Make sure that the intersection point lies on the same plane as the selected polygon as we only need to add it there and not
							// to any other edge that might intersect the cutting plane.

							if( SelectedPoly->OnPlane( (FVector)Intersection ) )
							{
								P->Vertices.Insert( Intersection, (v+1) % P->Vertices.Num() );
								break;
							}
						}
					}

					NewPolygons.Add( *P );
				}
			}

			// Replace the old polygon list with the new one
			Brush->Brush->Polys->Element = NewPolygons;
		}
	}
	else if( NumPolygonsSelected == 0 && NumEdgesSelected > 0 && NumVerticesSelected == 0 )
	{
		//
		// Splitting a brush at an edge mid point (ring cut)
		//

		// Get the selected edge
		TArray<FGeomEdge*> Edges;
		mode->GetSelectedEdges( Edges );
		check( Edges.Num() > 0 );

		FGeomEdge* Edge = Edges[0];

		// Generate a base and a normal for the cutting plane

		FGeomVertex* Vertex0 = &GeomObject->VertexPool[ Edge->VertexIndices[0] ];
		FGeomVertex* Vertex1 = &GeomObject->VertexPool[ Edge->VertexIndices[1] ];

		const FVector v0 = (FVector)*Vertex0->GetActualVertex( Vertex0->ActualVertexIndices[0] );
		const FVector v1 = (FVector)*Vertex1->GetActualVertex( Vertex1->ActualVertexIndices[0] );
		const FVector PlaneNormal( (v1 - v0).GetSafeNormal() );
		const FVector PlaneBase = 0.5f*(v1 + v0);

		ABrush* Brush = GeomObject->GetActualBrush();

		// The polygons for the new brush are stored in here and the polys inside of the original brush are replaced at the end of the loop

		TArray<FPoly> NewPolygons;

		// Clip each polygon against the cutting plane

		for( int32 p = 0 ; p < Brush->Brush->Polys->Element.Num() ; ++p )
		{
			FPoly* Poly = &Brush->Brush->Polys->Element[p];

			FPoly Front, Back;
			Front.Init();
			Back.Init();

			int32 Res = Poly->SplitWithPlane( (FVector3f)PlaneBase, (FVector3f)PlaneNormal, &Front, &Back, 1 );
			switch( Res )
			{
				case SP_Split:
					NewPolygons.Add( Front );
					NewPolygons.Add( Back );
					break;

				default:
					NewPolygons.Add( *Poly );
					break;
			}
		}

		// Replace the old polygon list with the new one
		Brush->Brush->Polys->Element = NewPolygons;
	}
	else if( NumPolygonsSelected == 1 && NumEdgesSelected == 0 && NumVerticesSelected == 2 )
	{
		//
		// Splitting a polygon across 2 vertices
		//

		// Get the selected verts
		TArray<FGeomVertex*> Verts;
		mode->GetSelectedVertices( Verts );
		check( Verts.Num() == 2 );

		FGeomVertex* Vertex0 = Verts[0];
		FGeomVertex* Vertex1 = Verts[1];

		const FVector v0 = (FVector)*Vertex0->GetActualVertex( Vertex0->ActualVertexIndices[0] );
		const FVector v1 = (FVector)*Vertex1->GetActualVertex( Vertex1->ActualVertexIndices[0] );

		// Get the selected polygon
		TArray<FGeomPoly*> Polys;
		mode->GetSelectedPolygons( Polys );
		check( Polys.Num() == 1 );

		FGeomPoly* SelectedPoly = Polys[0];
		FPoly* Poly = SelectedPoly->GetActualPoly();

		//
		// Sanity checking
		//
		{
			// 1. Make sure that the selected vertices are part of the selected polygon

			if( !SelectedPoly->GetActualPoly()->Vertices.Contains( (FVector3f)v0 ) || !SelectedPoly->GetActualPoly()->Vertices.Contains( (FVector3f)v1 ) )
			{
				GeomError( NSLOCTEXT("UnrealEd", "Error_SelectedVerticesMustBelongToSelectedPoly", "The vertices used for splitting must be part of the selected polygon.").ToString() );
				return false;
			}
		}

		// Generate a base and a normal for the cutting plane

		FVector v2 = v0 + (SelectedPoly->GetNormal() * 64.0f);

		const FPlane PlaneNormal( v0, v1, v2 );
		const FVector PlaneBase = 0.5f*(v1 + v0);

		ABrush* Brush = GeomObject->GetActualBrush();

		// The polygons for the new brush are stored in here and the polys inside of the original brush are replaced at the end of the loop

		TArray<FPoly> NewPolygons;

		// Clip the selected polygon against the cutting plane.

		for( int32 p = 0 ; p < Brush->Brush->Polys->Element.Num() ; ++p )
		{
			FPoly* P = &Brush->Brush->Polys->Element[p];

			if( P == Poly )
			{
				FPoly Front, Back;
				Front.Init();
				Back.Init();

				int32 Res = P->SplitWithPlane( (FVector3f)PlaneBase, (FVector3f)PlaneNormal, &Front, &Back, 1 );
				switch( Res )
				{
				case SP_Split:
					NewPolygons.Add( Front );
					NewPolygons.Add( Back );
					break;

				default:
					NewPolygons.Add( *P );
					break;
				}
			}
			else
			{
				NewPolygons.Add( *P );
			}
		}

		// Replace the old polygon list with the new one
		Brush->Brush->Polys->Element = NewPolygons;
	}
	else if( NumPolygonsSelected == 0 && NumEdgesSelected == 0 && NumVerticesSelected == 2 )
	{
		//
		// Splitting a brush across 2 vertices
		//

		// Get the selected verts
		TArray<FGeomVertex*> Verts;
		mode->GetSelectedVertices( Verts );
		check( Verts.Num() == 2 );

		// Generate a base and a normal for the cutting plane

		FGeomVertex* Vertex0 = Verts[0];
		FGeomVertex* Vertex1 = Verts[1];

		const FVector v0 = (FVector)*Vertex0->GetActualVertex( Vertex0->ActualVertexIndices[0] );
		const FVector v1 = (FVector)*Vertex1->GetActualVertex( Vertex1->ActualVertexIndices[0] );

		FVector v2 = ((Vertex0->GetNormal() + Vertex1->GetNormal()) / 2.0f) * 64.f;

		const FPlane PlaneNormal( v0, v1, v2 );
		const FVector PlaneBase = 0.5f*(v1 + v0);

		ABrush* Brush = GeomObject->GetActualBrush();

		// The polygons for the new brush are stored in here and the polys inside of the original brush are replaced at the end of the loop

		TArray<FPoly> NewPolygons;

		// Clip each polygon against the cutting plane

		for( int32 p = 0 ; p < Brush->Brush->Polys->Element.Num() ; ++p )
		{
			FPoly* Poly = &Brush->Brush->Polys->Element[p];

			FPoly Front, Back;
			Front.Init();
			Back.Init();

			int32 Res = Poly->SplitWithPlane( (FVector3f)PlaneBase, (FVector3f)PlaneNormal, &Front, &Back, 1 );
			switch( Res )
			{
			case SP_Split:
				NewPolygons.Add( Front );
				NewPolygons.Add( Back );
				break;

			default:
				NewPolygons.Add( *Poly );
				break;
			}
		}

		// Replace the old polygon list with the new one
		Brush->Brush->Polys->Element = NewPolygons;
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	bPendingPivotOffsetUpdate = true;
	return true;
}

UGeomModifier_Triangulate::UGeomModifier_Triangulate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Triangulate", "Triangulate");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Triangulate", "Tooltip", "Break the selected polygons down into triangles.");
	bPushButton = true;

	ToolbarIconName = TEXT("GeometryMode.TriangulateTool");
}

bool UGeomModifier_Triangulate::SupportsCurrentSelection()
{
	if ( !Super::SupportsCurrentSelection() )
	{
		return false;
	}

	FEdModeGeometry* Mode = (FEdModeGeometry*) GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && (!Mode->HaveEdgesSelected() && !Mode->HaveVerticesSelected());
}

bool UGeomModifier_Triangulate::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	bool bHavePolygonsSelected = mode->HavePolygonsSelected(); 

	// Mark the selected polygons so we can find them in the next loop, and create
	// a local list of FPolys to triangulate later.

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		TArray<FPoly> PolyList;

		for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
		{
			FGeomPoly* gp = &go->PolyPool[p];

			if( gp->IsSelected() || !bHavePolygonsSelected )
			{
				gp->GetParentObject()->GetActualBrush()->Brush->Polys->Element[ gp->ActualPolyIndex ].PolyFlags |= PF_GeomMarked;
				PolyList.Add( gp->GetParentObject()->GetActualBrush()->Brush->Polys->Element[ gp->ActualPolyIndex ] );
			}
		}

		// Delete existing polygons

		for( int32 p = 0 ; p < go->GetActualBrush()->Brush->Polys->Element.Num() ; ++p )
		{
			if( (go->GetActualBrush()->Brush->Polys->Element[ p ].PolyFlags&PF_GeomMarked) > 0 )
			{
				go->GetActualBrush()->Brush->Polys->Element.RemoveAt( p );
				p = -1;
			}
		}

		// Triangulate the old polygons into the brush

		for( int32 p = 0 ; p < PolyList.Num() ; ++p )
		{
			TArray<FPoly> Triangles;
			PolyList[p].Triangulate( go->GetActualBrush(), Triangles );

			for( int32 t = 0 ; t < Triangles.Num() ; ++t )
			{
				go->GetActualBrush()->Brush->Polys->Element.Add( Triangles[t] );
			}
		}
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	return true;
}

UGeomModifier_Optimize::UGeomModifier_Optimize(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Optimize", "Optimize");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Optimize", "Tooltip", "Optimizes the selected geometry by merging together any polygons which can be formed into a single convex polygon.");
	bPushButton = true;
	ToolbarIconName = TEXT("GeometryMode.OptimizeTool");
}

bool UGeomModifier_Optimize::OnApply()
{
	// First triangulate before performing optimize
	Super::OnApply();

	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	TArray<FPoly> Polygons;

	if( mode->HavePolygonsSelected() )
	{
		for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
		{
			FGeomObjectPtr go = *Itor;
			ABrush* ActualBrush = go->GetActualBrush();

			// Gather a list of polygons that are 
			for( int32 p = 0 ; p < go->PolyPool.Num() ; ++p )
			{
				FGeomPoly* gp = &go->PolyPool[p];

				if( gp->IsSelected() )
				{
					ActualBrush->Brush->Polys->Element[ gp->ActualPolyIndex ].PolyFlags |= PF_GeomMarked;
					Polygons.Add( ActualBrush->Brush->Polys->Element[ gp->ActualPolyIndex ] );
				}
			}

			// Delete existing polygons

			for( int32 p = 0 ; p < go->GetActualBrush()->Brush->Polys->Element.Num() ; ++p )
			{
				if( (ActualBrush->Brush->Polys->Element[ p ].PolyFlags&PF_GeomMarked) > 0 )
				{
					ActualBrush->Brush->Polys->Element.RemoveAt( p );
					p = -1;
				}
			}

			// Optimize the polygons in the list

			FPoly::OptimizeIntoConvexPolys( ActualBrush, Polygons );

			// Copy the new polygons into the brush

			for( int32 p = 0 ; p < Polygons.Num() ; ++p )
			{
				FPoly Poly = Polygons[p];

				Poly.PolyFlags &= ~PF_GeomMarked;

				ActualBrush->Brush->Polys->Element.Add( Poly );
			}
		}
	}
	else
	{
		for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
		{
			FGeomObjectPtr go = *Itor;
			ABrush* ActualBrush = go->GetActualBrush();

			// Optimize the polygons

			FPoly::OptimizeIntoConvexPolys( ActualBrush, ActualBrush->Brush->Polys->Element );
		}
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	return true;
}

UGeomModifier_Turn::UGeomModifier_Turn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Turn", "Turn");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Turn", "Tooltip", "Given a selected edge common to two triangles, turn the edge so that it is connected to the previously unconnected vertices.");
	bPushButton = true;

	ToolbarIconName = TEXT("GeometryMode.TurnTool");
}

bool UGeomModifier_Turn::SupportsCurrentSelection()
{
	FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && Mode->HaveEdgesSelected();
}

bool UGeomModifier_Turn::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	// Edges

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		TArray<FGeomEdge> Edges;
		go->CompileUniqueEdgeArray( &Edges );

		// Make sure that all polygons involved are triangles

		for( int32 e = 0 ; e < Edges.Num() ; ++e )
		{
			FGeomEdge* ge = &Edges[e];

			for( int32 p = 0 ; p < ge->ParentPolyIndices.Num() ; ++p )
			{
				FGeomPoly* gp = &go->PolyPool[ ge->ParentPolyIndices[p] ];
				FPoly* Poly = gp->GetActualPoly();

				if( Poly->Vertices.Num() != 3 )
				{
					FNotificationInfo NotificationInfo(LOCTEXT("Error_PolygonsOnEdgeToTurnMustBeTriangles", "The polygons on each side of the edge you want to turn must be triangles."));
					NotificationInfo.ExpireDuration = 3.0f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					EndTrans();
					return 0;
				}
			}
		}

		// Turn the edges, one by one

		for( int32 e = 0 ; e < Edges.Num() ; ++e )
		{
			FGeomEdge* ge = &Edges[e];

			TArray<FVector> Quad;

			// Since we're doing each edge individually, they should each have exactly 2 polygon
			// parents (and each one is a triangle (verified above))

			if( ge->ParentPolyIndices.Num() == 2 )
			{
				FGeomPoly* gp = &go->PolyPool[ ge->ParentPolyIndices[0] ];
				FPoly* Poly = gp->GetActualPoly();
				FPoly SavePoly0 = *Poly;

				int32 idx0 = Poly->GetVertexIndex( go->VertexPool[ ge->VertexIndices[0] ] );
				int32 idx1 = Poly->GetVertexIndex( go->VertexPool[ ge->VertexIndices[1] ] );
				int32 idx2 = INDEX_NONE;

				if( idx0 + idx1 == 1 )
				{
					idx2 = 2;
				}
				else if( idx0 + idx1 == 3 )
				{
					idx2 = 0;
				}
				else
				{
					idx2 = 1;
				}

				Quad.Add( (FVector)Poly->Vertices[idx0] );
				Quad.Add( (FVector)Poly->Vertices[idx2] );
				Quad.Add( (FVector)Poly->Vertices[idx1] );

				gp = &go->PolyPool[ ge->ParentPolyIndices[1] ];
				Poly = gp->GetActualPoly();
				FPoly SavePoly1 = *Poly;

				for( int32 v = 0 ; v < Poly->Vertices.Num() ; ++v )
				{
					Quad.AddUnique( (FVector)Poly->Vertices[v] );
				}

				// If the adjoining polys were coincident, don't try to turn the edge
				if (Quad.Num() == 3)
				{
					continue;
				}

				// Create new polygons

				FPoly* NewPoly;

				NewPoly = new( gp->GetParentObject()->GetActualBrush()->Brush->Polys->Element )FPoly();

				NewPoly->Init();
				new(NewPoly->Vertices) FVector3f(Quad[2]);
				new(NewPoly->Vertices) FVector3f(Quad[1]);
				new(NewPoly->Vertices) FVector3f(Quad[3]);

				NewPoly->Base = SavePoly0.Base;
				NewPoly->Material = SavePoly0.Material;
				NewPoly->PolyFlags = SavePoly0.PolyFlags;
				NewPoly->TextureU = SavePoly0.TextureU;
				NewPoly->TextureV = SavePoly0.TextureV;
				NewPoly->Normal = FVector3f::ZeroVector;
				NewPoly->Finalize(go->GetActualBrush(),1);

				NewPoly = new( gp->GetParentObject()->GetActualBrush()->Brush->Polys->Element )FPoly();

				NewPoly->Init();
				new(NewPoly->Vertices) FVector3f(Quad[3]);
				new(NewPoly->Vertices) FVector3f(Quad[1]);
				new(NewPoly->Vertices) FVector3f(Quad[0]);

				NewPoly->Base = SavePoly1.Base;
				NewPoly->Material = SavePoly1.Material;
				NewPoly->PolyFlags = SavePoly1.PolyFlags;
				NewPoly->TextureU = SavePoly1.TextureU;
				NewPoly->TextureV = SavePoly1.TextureV;
				NewPoly->Normal = FVector3f::ZeroVector;
				NewPoly->Finalize(go->GetActualBrush(),1);

				// Tag the old polygons

				for( int32 p = 0 ; p < ge->ParentPolyIndices.Num() ; ++p )
				{
					FGeomPoly* GeomPoly = &go->PolyPool[ ge->ParentPolyIndices[p] ];

					go->GetActualBrush()->Brush->Polys->Element[GeomPoly->ActualPolyIndex].PolyFlags |= PF_GeomMarked;
				}
			}
		}

		// Delete the old polygons

		for( int32 p = 0 ; p < go->GetActualBrush()->Brush->Polys->Element.Num() ; ++p )
		{
			if( (go->GetActualBrush()->Brush->Polys->Element[ p ].PolyFlags&PF_GeomMarked) > 0 )
			{
				go->GetActualBrush()->Brush->Polys->Element.RemoveAt( p );
				p = -1;
			}
		}
	}

	mode->FinalizeSourceData();
	mode->GetFromSource();

	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	return true;
}

UGeomModifier_Weld::UGeomModifier_Weld(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = NSLOCTEXT("UnrealEd", "Weld", "Weld");
	Tooltip = NSLOCTEXT("UnrealEd.GeomModifier_Weld", "Tooltip", "Merge all selected vertices to the first selected vertex.");
	bPushButton = true;
	ToolbarIconName = TEXT("GeometryMode.WeldTool");
}


bool UGeomModifier_Weld::SupportsCurrentSelection()
{
	FEdModeGeometry* Mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);
	return Mode != nullptr && (Mode->HaveVerticesSelected() && !Mode->HaveEdgesSelected() && !Mode->HavePolygonsSelected());
}

bool UGeomModifier_Weld::OnApply()
{
	FEdModeGeometry* mode = (FEdModeGeometry*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_Geometry);

	// Verts

	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;

		go->CompileSelectionOrder();

		if( go->SelectionOrder.Num() > 1 )
		{
			//NOTE: function assumes ONLY vertices are selected, UGeomModifier_Weld::Supports must ensure this.
			FGeomVertex* FirstSel = (FGeomVertex*)go->SelectionOrder[0];

			// Move all selected vertices to the location of the first vertex that was selected.
			for( int32 v = 1 ; v < go->SelectionOrder.Num() ; ++v )
			{
				FGeomVertex* gv = (FGeomVertex*)go->SelectionOrder[v];

				if( gv->IsSelected() )
				{
					gv->X = FirstSel->X;
					gv->Y = FirstSel->Y;
					gv->Z = FirstSel->Z;
				}
			}

			go->SendToSource();
		}
	}

	
	mode->FinalizeSourceData();
	mode->GetFromSource();

	
	GEditor->RebuildAlteredBSP(); // Brush has been altered, update the Bsp

	//finally, cache the selections AFTER the weld and set the widget to the appropriate selection
	for( FEdModeGeometry::TGeomObjectIterator Itor( mode->GeomObjectItor() ) ; Itor ; ++Itor )
	{
		FGeomObjectPtr go = *Itor;
		go->CompileSelectionOrder();

		ABrush* Actor = go->GetActualBrush();

		StoreCurrentGeomSelections( Actor->SavedSelections , go );

		go->SelectNone();
		int32 res = go->SetPivotFromSelectionArray( Actor->SavedSelections );
		if( res == INDEX_NONE )
		{
			FEditorModeTools& Tools = GLevelEditorModeTools();
			Tools.SetPivotLocation( Actor->GetActorLocation() , false );
		}
		go->ForceLastSelectionIndex( res );
	}
	return true;
}

#undef LOCTEXT_NAMESPACE 
