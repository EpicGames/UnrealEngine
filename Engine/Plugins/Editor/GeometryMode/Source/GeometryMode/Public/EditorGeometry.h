// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ICursor.h"
#include "UObject/GCObject.h"
#include "HitProxies.h"
#include "Math/Rotator.h"

class ABrush;
class FGeomObject;
class FPoly;

typedef TSharedPtr<class FGeomObject> FGeomObjectPtr;


/**
 * Base class for all classes that support storing and editing of geometry.
 */
class FGeomBase
{
public:
	FGeomBase();
	virtual ~FGeomBase() = default;

	/** Does nothing if not in geometry mode.*/
	virtual void Select( bool InSelect = 1 );

	bool IsSelected() const		{ return SelectionIndex > INDEX_NONE; }
	int32 GetSelectionIndex() const	{ return SelectionIndex; }

	/**
	 * Allows manual setting of the selection index.  This is dangerous and should
	 * only be called by the FEdModeGeometry::PostUndo() function.
	 */
	void ForceSelectionIndex( int32 InSelectionIndex )	{ SelectionIndex = InSelectionIndex; }

	/** Returns a location that represents the middle of the object */
	virtual FVector GetMidPoint() const = 0;

	/** Returns a valid position for the widget to be drawn at for this object */
	virtual FVector GetWidgetLocation()					{ return FVector::ZeroVector; }

	/** Gets the local rotation to use when transforming this object */
	virtual FRotator GetWidgetRotation() const			{ return Normal.Rotation(); }

	//@{
	FGeomObjectPtr GetParentObject();
	const FGeomObjectPtr GetParentObject() const;
	//@}

	/** Returns true if this geometry objects is a vertex. */
	virtual bool IsVertex() const						{ return 0; }

	//@{
	const FVector& GetNormal() const					{ return Normal; }
	const FVector& GetMid() const						{ return Mid; }

	int32 GetParentObjectIndex() const					{ return ParentObjectIndex; }
	//@}

	//@{
	void SetNormal(const FVector& InNormal)				{ Normal = InNormal; }
	void SetMid(const FVector& InMid)					{ Mid = InMid; }

	void SetParentObjectIndex(int32 InParentObjectIndex)	{ ParentObjectIndex = InParentObjectIndex; }
	//@}

protected:
	/** Allows polys, edges and vertices to point to the FGeomObject that owns them. */
	int32 ParentObjectIndex;

	/** The normal vector for this object. */
	FVector Normal;			

	/** The mid point for this object. */
	FVector Mid;			

private:
	/** If > INDEX_NONE, this object is selected */
	int32 SelectionIndex;		
};

///////////////////////////////////////////////////////////////////////////////

/**
 * An index pair denoting a polygon and vertex within the parent objects ABrush.
 */
struct FPolyVertexIndex
{
	FPolyVertexIndex()
	{
		PolyIndex = VertexIndex = INDEX_NONE;
	}
	FPolyVertexIndex( int32 InPolyIndex, int32 InVertexIndex )
	{
		PolyIndex = InPolyIndex;
		VertexIndex = InVertexIndex;
	}

	int32 PolyIndex, VertexIndex;

	bool operator==( const FPolyVertexIndex& In ) const
	{
		if( In.PolyIndex != PolyIndex
				|| In.VertexIndex != VertexIndex )
		{
			return 0;
		}

		return 1;
	}
};

///////////////////////////////////////////////////////////////////////////////

/**
 * A 3D position.
 */
class FGeomVertex : public FGeomBase, public FVector3f
{
public:
	FGeomVertex();

	virtual FVector GetWidgetLocation();

	virtual FRotator GetWidgetRotation() const;

	virtual FVector GetMidPoint() const;

	virtual bool IsVertex() const		{ return 1; }

	/** The list of vertices that this vertex represents. */
	TArray<FPolyVertexIndex> ActualVertexIndices;
	FVector3f* GetActualVertex( FPolyVertexIndex& InPVI );

	/**
	 * Indices into the parent poly pool. A vertex can belong to more 
	 * than one poly and this keeps track of that relationship.
	 */
	TArray<int32> ParentPolyIndices;

	/** Assignment simply copies off the vertex position. */
	FGeomVertex& operator=( const FVector3f& In )
	{
		X = In.X;
		Y = In.Y;
		Z = In.Z;
		return *this;
	}
};

///////////////////////////////////////////////////////////////////////////////

/**
 * The space between 2 adjacent vertices.
 */
class FGeomEdge : public FGeomBase
{
public:
	FGeomEdge();
	
	virtual FVector GetWidgetLocation();

	virtual FVector GetMidPoint() const;

	/** Returns true if InEdge matches this edge, independant of winding. */
	bool IsSameEdge( const FGeomEdge& InEdge ) const;

	/**
	 * Indices into the parent poly pool. An edge can belong to more 
	 * than one poly and this keeps track of that relationship.
	 */
	TArray<int32> ParentPolyIndices;

	/** Indices into the parent vertex pool. */
	int32 VertexIndices[2];
};

///////////////////////////////////////////////////////////////////////////////

/**
 * An individual polygon.
 */
class FGeomPoly : public FGeomBase
{
public:
	FGeomPoly();

	virtual FVector GetWidgetLocation();

	virtual FVector GetMidPoint() const;

	/**
	 * The polygon this poly represents.  This is an index into the polygon array inside 
	 * the ABrush that is selected in this objects parent FGeomObject.
	 */
	int32 ActualPolyIndex;
	FPoly* GetActualPoly();

	/** Array of indices into the parent objects edge pool. */
	TArray<int> EdgeIndices;

	bool operator==( const FGeomPoly& In ) const
	{
		if( In.ActualPolyIndex != ActualPolyIndex
				|| In.ParentObjectIndex != ParentObjectIndex
				|| In.EdgeIndices.Num() != EdgeIndices.Num() )
		{
			return 0;
		}

		for( int32 x = 0 ; x < EdgeIndices.Num() ; ++x )
		{
			if( EdgeIndices[x] != In.EdgeIndices[x] )
			{
				return 0;
			}
		}

		return 1;
	}
};

///////////////////////////////////////////////////////////////////////////////

/**
 * A group of polygons forming an individual object.
 */
class FGeomObject : public FGeomBase, public FGCObject
{
public:
	FGeomObject();
	
	virtual FVector GetMidPoint() const override;

	/** Index to the ABrush actor this object represents. */
	TObjectPtr<ABrush> ActualBrush;
	ABrush* GetActualBrush()				{ return ActualBrush; }
	const ABrush* GetActualBrush() const	{ return ActualBrush; }

	/**
	 * Used for tracking the order of selections within this object.  This 
	 * is required for certain modifiers to work correctly.
	 *
	 * This list is generated on the fly whenever the array is accessed but
	 * is marked as dirty (see bSelectionOrderDirty).
	 *
	 * NOTE: do not serialize
	 */
	TArray<FGeomBase*> SelectionOrder;

	/** @name Dirty seleciton order
	 * When the selection order is dirty, the selection order array needs to be compiled before
	 * being accessed.  This should NOT be serialized.
	 */
	//@{
	/** Dirties the selection order. */
	void DirtySelectionOrder()			{ bSelectionOrderDirty = 1; }

	/** If 1, the selection order array needs to be compiled before being accessed.
	 *
	 * NOTE: do not serialize
	 */
	bool bSelectionOrderDirty;
	//@}

	void CompileSelectionOrder();

	/** Main lists.  All lower data types refer to the contents of these pools through indices. */
	TArray<FGeomVertex> VertexPool;
	TArray<FGeomEdge> EdgePool;
	TArray<FGeomPoly> PolyPool;

	/**
	 * Compiles a list of unique edges.  This runs through the edge pool
	 * and only adds edges into the pool that aren't already there (the
	 * difference being that this routine counts edges that share the same
	 * vertices, but are wound backwards to each other, as being equal).
	 *
	 * @param	InEdges		The edge array to fill up.
	 */
	void CompileUniqueEdgeArray( TArray<FGeomEdge>* InEdges );

	/** Tells the object to recompute all of it's internal data. */
	void ComputeData();

	/** Erases all current data for this object. */
	void ClearData();

	virtual FVector GetWidgetLocation() override;
	int32 AddVertexToPool( int32 InObjectIndex, int32 InParentPolyIndex, int32 InPolyIndex, int32 InVertexIndex );
	int32 AddEdgeToPool( FGeomPoly* InPoly, int32 InParentPolyIndex, int32 InVectorIdxA, int32 InVectorIdxB );
	virtual void GetFromSource();
	virtual void SendToSource();
	bool FinalizeSourceData();
	int32 GetObjectIndex();

	void SelectNone();
	int32 GetNewSelectionIndex();

	/**
	 * Allows manual setting of the last selection index.  This is dangerous and should
	 * only be called by the FEdModeGeometry::PostUndo() function.
	 */
	void ForceLastSelectionIndex( int32 InLastSelectionIndex ) { LastSelectionIndex = InLastSelectionIndex; }

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override;
	
	/**
	 * Set the pivot position based on the 'highest' selected object (vertex/edge/poly) in the given selection array
	 * @param SelectionArray - The array of selections
	 * @return The index of the array entry used for setting the pivot. Returns INDEX_NONE if pivot not set
	 */
	int32 SetPivotFromSelectionArray( TArray<struct FGeomSelection>& SelectionArray );

	/**
	 * Update the selection state based on the passed in array
	 */
	void UpdateFromSelectionArray( TArray<struct FGeomSelection>& SelectionArray );

private:
	int32 LastSelectionIndex;
};




/*-----------------------------------------------------------------------------
	Hit proxies for geometry editing.
-----------------------------------------------------------------------------*/

/**
 * Hit proxy for polygons. 
 */
struct HGeomPolyProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	TWeakPtr<FGeomObject> GeomObjectWeakPtr;
	int32				PolyIndex;

	HGeomPolyProxy(FGeomObjectPtr InGeomObject, int32 InPolyIndex):
		HHitProxy(HPP_UI),
		GeomObjectWeakPtr(InGeomObject),
		PolyIndex(InPolyIndex)
	{}

	FGeomObject* GetGeomObject() { return GeomObjectWeakPtr.IsValid() ? GeomObjectWeakPtr.Pin().Get() : nullptr; }

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}

	/**
	 * Method that specifies whether the hit proxy *always* allows translucent primitives to be associated with it or not,
	 * regardless of any other engine/editor setting. For example, if translucent selection was disabled, any hit proxies
	 * returning true would still allow translucent selection. In this specific case, true is always returned because geometry
	 * mode hit proxies always need to be selectable or geometry mode will not function correctly.
	 *
	 * @return	true if translucent primitives are always allowed with this hit proxy; false otherwise
	 */
	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};

/**
 * Hit proxy for edges. 
 */
struct HGeomEdgeProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	// Kept around for backwards compat
	TWeakPtr<FGeomObject> GeomObjectWeakPtr;
	int32				EdgeIndex;

	HGeomEdgeProxy(FGeomObjectPtr InGeomObject, int32 InEdgeIndex):
		HHitProxy(HPP_UI),
		GeomObjectWeakPtr(InGeomObject),
		EdgeIndex(InEdgeIndex)
	{}

	FGeomObject* GetGeomObject() { return GeomObjectWeakPtr.IsValid() ? GeomObjectWeakPtr.Pin().Get() : nullptr; }

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}

	/**
	 * Method that specifies whether the hit proxy *always* allows translucent primitives to be associated with it or not,
	 * regardless of any other engine/editor setting. For example, if translucent selection was disabled, any hit proxies
	 * returning true would still allow translucent selection. In this specific case, true is always returned because geometry
	 * mode hit proxies always need to be selectable or geometry mode will not function correctly.
	 *
	 * @return	true if translucent primitives are always allowed with this hit proxy; false otherwise
	 */
	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Hit proxy for vertices.
 */
struct HGeomVertexProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	TWeakPtr<FGeomObject> GeomObjectWeakPtr;
	int32				VertexIndex;

	HGeomVertexProxy(FGeomObjectPtr InGeomObject, int32 InVertexIndex):
		HHitProxy(HPP_UI),
		GeomObjectWeakPtr(InGeomObject),
		VertexIndex(InVertexIndex)
	{}

	FGeomObject* GetGeomObject() { return GeomObjectWeakPtr.IsValid() ? GeomObjectWeakPtr.Pin().Get() : nullptr; }

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}

	/**
	 * Method that specifies whether the hit proxy *always* allows translucent primitives to be associated with it or not,
	 * regardless of any other engine/editor setting. For example, if translucent selection was disabled, any hit proxies
	 *  returning true would still allow translucent selection. In this specific case, true is always returned because geometry
	 * mode hit proxies always need to be selectable or geometry mode will not function correctly.
	 *
	 * @return	true if translucent primitives are always allowed with this hit proxy; false otherwise
	 */
	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};
