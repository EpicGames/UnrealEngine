// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorModeTools.h"
#include "EdMode.h"

class FCanvas;
class FEditorViewportClient;
class FGeomBase;
class FGeomEdge;
class FGeomObject;
class FGeomPoly;
class FGeomVertex;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class UGeomModifier;
struct FConvexVolume;

class FGeomPoly;
class FGeomEdge;
class FGeomVertex;
class FGeomBase;
class FGeomObject;
class ABrush;
typedef TSharedPtr<class FGeomObject> FGeomObjectPtr;


/**
 * Allows for BSP geometry to be edited directly
 */
class FEdModeGeometry : public FEdMode
{
public:

	FEdModeGeometry();

	/**
	 * Struct for cacheing of selected objects components midpoints for reselection when rebuilding the BSP
	 */
	struct HGeomMidPoints
	{
		/** The actor that the verts/edges/polys belong to */
		ABrush* ActualBrush;

		/** Arrays of the midpoints of all the selected verts/edges/polys */
		TArray<FVector> VertexPool;
		TArray<FVector> EdgePool;
		TArray<FVector> PolyPool;	
	};

	static TSharedRef< FEdModeGeometry > Create();
	virtual ~FEdModeGeometry();

	// FEdMode interface
	virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI) override;
	virtual bool ShowModeWidgets() const override;
	virtual bool UsesToolkits() const override;
	virtual bool ShouldDrawBrushWireframe( AActor* InActor ) const override;
	virtual bool GetCustomDrawingCoordinateSystem( FMatrix& InMatrix, void* InData ) override;
	virtual bool GetCustomInputCoordinateSystem( FMatrix& InMatrix, void* InData ) override;
	virtual void Enter() override;
	virtual void Exit() override;
	virtual void ActorSelectionChangeNotify() override;
	virtual void MapChangeNotify() override;
	virtual void SelectionChanged() override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const override;
	// End of FEdMode interface

	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	// End of FGCObject interface

	virtual void GeometrySelectNone(bool bStoreSelection, bool bResetPivot);
	
	/**
	* Returns the number of objects that are selected.
	*/
	virtual int32 CountObjectsSelected();

	/**
	* Returns the number of polygons that are selected.
	*/
	virtual int32 CountSelectedPolygons();

	/**
	* Returns the polygons that are selected.
	*
	* @param	InPolygons	An array to fill with the selected polygons.
	*/
	virtual void GetSelectedPolygons( TArray<FGeomPoly*>& InPolygons ) const;

	/**
	* Returns true if the user has polygons selected.
	*/
	virtual bool HavePolygonsSelected();

	/**
	* Returns the number of edges that are selected.
	*/
	virtual int32 CountSelectedEdges();

	/**
	* Returns the edges that are selected.
	*
	* @param	InEdges	An array to fill with the selected edges.
	*/
	virtual void GetSelectedEdges( TArray<FGeomEdge*>& InEdges ) const;

	/**
	* Returns true if the user has edges selected.
	*/
	virtual bool HaveEdgesSelected();

	/**
	* Returns the number of vertices that are selected.
	*/
	virtual int32 CountSelectedVertices();
	
	/**
	* Returns true if the user has vertices selected.
	*/
	virtual bool HaveVerticesSelected();

	/**
	 * Fills an array with all selected vertices.
	 */
	virtual void GetSelectedVertices( TArray<FGeomVertex*>& InVerts ) const;

	/**
	 * Utility function that allow you to poll and see if certain sub elements are currently selected.
	 *
	 * Returns a combination of the flags in EGeomSelectionStatus.
	 */
	virtual int32 GetSelectionState();

	/**
	 * Cache all the selected geometry on the object, and add to the array if any is found
	 *
	 * Return true if new object has been added to the array.
	 */
	bool CacheSelectedData( TArray<HGeomMidPoints>& raGeomData, const FGeomObject& rGeomObject ) const;

	/**
	 * Attempt to find all the new geometry using the cached data, and cache those new ones out
	 *
	 * Return true everything was found (or there was nothing to find)
	 */
	bool FindFromCache( TArray<HGeomMidPoints>& raGeomData, FGeomObject& rGeomObject, TArray<FGeomBase*>& raSelectedGeom ) const;

	/**
	 * Select all the verts/edges/polys that were found
	 *
	 * Return true if successful
	 */
	bool SelectCachedData( TArray<FGeomBase*>& raSelectedGeom ) const;

	/**
	 * Compiles geometry mode information from the selected brushes.
	 */
	virtual void GetFromSource();
	
	/**
	 * Changes the source brushes to match the current geometry data.
	 */
	virtual void SendToSource();
	virtual bool FinalizeSourceData();
	virtual void PostUndo() override;
	virtual bool ExecDelete();
	virtual void UpdateInternalData() override;

	void RenderPoly( const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI );
	void RenderEdge( const FSceneView* View, FPrimitiveDrawInterface* PDI );
	void RenderVertex( const FSceneView* View, FPrimitiveDrawInterface* PDI );

	void ShowModifierWindow(bool bShouldShow);

	/** @name GeomObject iterators */
	//@{
	typedef TArray<FGeomObjectPtr>::TIterator TGeomObjectIterator;
	typedef TArray<FGeomObjectPtr>::TConstIterator TGeomObjectConstIterator;

	TGeomObjectIterator			GeomObjectItor()			{ return TGeomObjectIterator( GeomObjects ); }
	TGeomObjectConstIterator	GeomObjectConstItor() const	{ return TGeomObjectConstIterator( GeomObjects ); }

	// @todo DB: Get rid of these; requires changes to FGeomBase::ParentObjectIndex
	FGeomObjectPtr GetGeomObject(int32 Index)					{ return GeomObjects[ Index ]; }
	const FGeomObjectPtr GetGeomObject(int32 Index) const		{ return GeomObjects[ Index ]; }
	//@}

protected:
	/** 
	 * Custom data compiled when this mode is entered, based on currently
	 * selected brushes.  This data is what is drawn and what the LD
	 * interacts with while in this mode.  Changes done here are
	 * reflected back to the real data in the level at specific times.
	 */
	TArray<FGeomObjectPtr> GeomObjects;
};



class UGeomModifier;

/**
 * Widget manipulation of geometry.
 */
class FModeTool_GeometryModify : public FModeTool
{
public:
	FModeTool_GeometryModify();

	virtual FString GetName() const override { return TEXT("Modifier"); }

	void SetCurrentModifier( UGeomModifier* InModifier );
	UGeomModifier* GetCurrentModifier() const;

	int32 GetNumModifiers() const;

	/**
	 * @return		true if the delta was handled by this editor mode tool.
	 */
	virtual bool InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale) override;

	virtual bool StartModify() override;
	virtual bool EndModify() override;

	virtual void StartTrans() override;
	virtual void EndTrans() override;

	virtual void SelectNone() override;
	virtual bool BoxSelect( FBox& InBox, bool InSelect = true ) override;
	virtual bool FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true ) override;

	/** @name Modifier iterators */
	//@{
	typedef TArray<TObjectPtr<UGeomModifier>>::TIterator TModifierIterator;
	typedef TArray<TObjectPtr<UGeomModifier>>::TConstIterator TModifierConstIterator;

	TModifierIterator		ModifierIterator()				{ return TModifierIterator( Modifiers ); }
	TModifierConstIterator	ModifierConstIterator() const	{ return TModifierConstIterator( Modifiers ); }

	// @todo DB: Get rid of these; requires changes to EditorGeometry.cpp
	UGeomModifier* GetModifier(int32 Index)					{ return Modifiers[ Index ]; }
	const UGeomModifier* GetModifier(int32 Index) const		{ return Modifiers[ Index ]; }
	//@}

	virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime) override;

	/** @return		true if the key was handled by this editor mode tool. */
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;

	virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas) override;
	
	/**
	 * Store the current geom selections for all geom objects
	 */
	void StoreAllCurrentGeomSelections();

	/** Used to track when actualy modification takes place */
	bool bGeomModified;

protected:
	/** All available modifiers. */
	TArray<TObjectPtr<UGeomModifier>> Modifiers;

	/** The current modifier. */
	UGeomModifier* CurrentModifier;	
};
