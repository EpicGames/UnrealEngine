// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#define UE_API GEOMETRYMODE_API

class FName;

typedef FName FEditorModeID;

struct FGeometryEditingModes
{
	static UE_API FEditorModeID EM_Geometry;
	static UE_API FEditorModeID EM_Bsp;
	static UE_API FEditorModeID EM_TextureAlign;
};
/**
 * Geometry mode module
 */
class FGeometryModeModule : public IModuleInterface
{
public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface
};

#undef UE_API
