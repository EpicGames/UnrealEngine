// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureAlignEdMode.h"
#include "EditorModeRegistry.h"
#include "EditorViewportClient.h"
#include "Engine/Brush.h"
#include "Modules/ModuleManager.h"
#include "EditorModeManager.h"
#include "ScopedTransaction.h"
#include "SurfaceIterators.h"
#include "EditorSupportDelegates.h"
#include "Engine/Polys.h"
#include "GeometryModeModule.h"

IMPLEMENT_MODULE( FTextureAlignModeModule, TextureAlignMode );

DEFINE_LOG_CATEGORY_STATIC(LogTextureAlignMode, Log, All);


void FTextureAlignModeModule::StartupModule()
{
	FEditorModeRegistry::Get().RegisterMode<FEdModeTexture>(
		FGeometryEditingModes::EM_TextureAlign,
		NSLOCTEXT("EditorModes", "TextureAlignmentMode", "Texture Alignment")
		);
}

void FTextureAlignModeModule::ShutdownModule()
{
	FEditorModeRegistry::Get().UnregisterMode(FGeometryEditingModes::EM_TextureAlign);
}

/*------------------------------------------------------------------------------
	Texture
------------------------------------------------------------------------------*/

FEdModeTexture::FEdModeTexture()
	:	ScopedTransaction( NULL )
	, TrackingWorld( NULL )
{
	Tools.Add( new FModeTool_Texture() );
	SetCurrentTool( MT_Texture );
}

FEdModeTexture::~FEdModeTexture()
{
	// Ensure no transaction is outstanding.
	check( !ScopedTransaction );
}

void FEdModeTexture::Enter()
{
	FEdMode::Enter();

	const bool bGetRawValue = true;
	SaveCoordSystem = GLevelEditorModeTools().GetCoordSystem(bGetRawValue);
	GLevelEditorModeTools().SetCoordSystem(COORD_Local);
}

void FEdModeTexture::Exit()
{
	if( ScopedTransaction != NULL )
	{
		delete ScopedTransaction;
		ScopedTransaction = NULL;
	}

	FEdMode::Exit();

	GLevelEditorModeTools().SetCoordSystem(SaveCoordSystem);
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

FVector FEdModeTexture::GetWidgetLocation() const
{
	for ( TSelectedSurfaceIterator<> It(GetWorld()) ; It ; ++It )
	{
		FBspSurf* Surf = *It;
		ABrush* BrushActor = ( ABrush* )Surf->Actor;
		if( BrushActor )
		{
			FPoly* poly = &BrushActor->Brush->Polys->Element[ Surf->iBrushPoly ];
			return BrushActor->ActorToWorld().TransformPosition( poly->GetMidPoint() );
		}
	}

	return FEdMode::GetWidgetLocation();
}

bool FEdModeTexture::ShouldDrawWidget() const
{
	return true;
}

bool FEdModeTexture::GetCustomDrawingCoordinateSystem( FMatrix& InMatrix, void* InData )
{
	// Texture mode is ALWAYS in local space
	GLevelEditorModeTools().SetCoordSystem(COORD_Local);

	FPoly* poly = NULL;

	for ( TSelectedSurfaceIterator<> It(GetWorld()) ; It ; ++It )
	{
		FBspSurf* Surf = *It;
		ABrush* BrushActor = ( ABrush* )Surf->Actor;
		if( BrushActor )
		{
			poly = &BrushActor->Brush->Polys->Element[ Surf->iBrushPoly ];
			break;
		}
	}

	if( !poly )
	{
		return false;
	}

	InMatrix = FMatrix::Identity;

	InMatrix.SetAxis( 2, (FVector)poly->Normal );
	InMatrix.SetAxis( 0, (FVector)poly->TextureU );
	InMatrix.SetAxis( 1, (FVector)poly->TextureV );

	InMatrix.RemoveScaling();

	return true;
}

bool FEdModeTexture::GetCustomInputCoordinateSystem( FMatrix& InMatrix, void* InData )
{
	return false;
}

EAxisList::Type FEdModeTexture::GetWidgetAxisToDraw( UE::Widget::EWidgetMode InWidgetMode ) const
{
	switch( InWidgetMode )
	{
	case UE::Widget::WM_Translate:
	case UE::Widget::WM_Scale:
		return EAxisList::XY;

	case UE::Widget::WM_Rotate:
		return EAxisList::Z;
	}

	return EAxisList::XYZ;
}

bool FEdModeTexture::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	// call base version because it calls the StartModify() virtual method needed to track drag events
	bool BaseRtn = FEdMode::StartTracking(InViewportClient, InViewport);

	// Complete the previous transaction if one exists
	if( ScopedTransaction )
	{
		EndTracking(InViewportClient, InViewport);
	}
	// Start a new transaction
	ScopedTransaction = new FScopedTransaction( NSLOCTEXT("UnrealEd", "TextureManipulation", "Texture Manipulation") );

	for( FConstLevelIterator Iterator = GetWorld()->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		Model->ModifySelectedSurfs( true );
	}

	return BaseRtn;
}

bool FEdModeTexture::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	// Clean up the scoped transaction if one is still pending
	if( ScopedTransaction != NULL )
	{
		delete ScopedTransaction;
		ScopedTransaction = NULL;
	}

	if( TrackingWorld )
	{
		TrackingWorld->MarkPackageDirty();
		ULevel::LevelDirtiedEvent.Broadcast();
		TrackingWorld = NULL;
	}

	// call base version because it calls the EndModify() virtual method needed to track drag events 
	return FEdMode::EndTracking(InViewportClient, InViewport);
}

bool FEdModeTexture::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return OtherModeID == FGeometryEditingModes::EM_Bsp;
}




/*-----------------------------------------------------------------------------
	FModeTool_TextureAlign.
-----------------------------------------------------------------------------*/

FModeTool_Texture::FModeTool_Texture()
{
	ID = MT_Texture;
	bUseWidget = 1;
	PreviousInputDrag = FVector::ZeroVector;
}

/**
 * @return		true if the delta was handled by this editor mode tool.
 */
bool FModeTool_Texture::InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale)
{
	if( InViewportClient->GetCurrentWidgetAxis() == EAxisList::None )
	{
		return false;
	}

	// calculate delta drag for this tick for the call to GEditor->polyTexPan below which is using relative (delta) mode
	FVector deltaDrag = InDrag;
	if (true == InViewportClient->IsPerspective())
	{
		// perspective viewports pass the absolute drag so subtract the last tick's drag value to get the delta
		deltaDrag -= PreviousInputDrag;
	}
	PreviousInputDrag = InDrag;

	if( !deltaDrag.IsZero() )
	{
		// Ensure each polygon has a unique base point index.
		for( FConstLevelIterator Iterator = InViewportClient->GetWorld()->GetLevelIterator(); Iterator; ++Iterator )
		{
			UModel* Model = (*Iterator)->Model;
			for(int32 SurfaceIndex = 0;SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
			{
				FBspSurf& Surf = Model->Surfs[SurfaceIndex];

				if(Surf.PolyFlags & PF_Selected)
				{
					const FVector3f Base = Model->Points[Surf.pBase];
					Surf.pBase = Model->Points.Add(Base);
				}
			}
			FMatrix Mat = GLevelEditorModeTools().GetCustomDrawingCoordinateSystem();
			FVector UVW = Mat.InverseTransformVector( deltaDrag );  // InverseTransformNormal because Mat is the transform from the surface/widget's coords to world coords
			const int32 X = static_cast<int32>(UVW.X);
			const int32 Y = static_cast<int32>(UVW.Y);
			constexpr int32 Z = 0;
			GEditor->polyTexPan( Model, X, Y, Z );  // 0 is relative mode because UVW is made from deltaDrag - the user input since the last tick
		}
	}

	if( !InRot.IsZero() )
	{
		const FRotationMatrix RotationMatrix( InRot );

		// Ensure each polygon has unique texture vector indices.
		for ( TSelectedSurfaceIterator<> It(InViewportClient->GetWorld()) ; It ; ++It )
		{
			FBspSurf* Surf = *It;
			UModel* Model = It.GetModel();

			FVector	TextureU = (FVector)Model->Vectors[Surf->vTextureU];
			FVector TextureV = (FVector)Model->Vectors[Surf->vTextureV];

			TextureU = RotationMatrix.TransformPosition( TextureU );
			TextureV = RotationMatrix.TransformPosition( TextureV );

			Surf->vTextureU = Model->Vectors.Add((FVector3f)TextureU);
			Surf->vTextureV = Model->Vectors.Add((FVector3f)TextureV);

			const bool bUpdateTexCoords = true;
			const bool bOnlyRefreshSurfaceMaterials = true;
			GEditor->polyUpdateBrush(Model, It.GetSurfaceIndex(), bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
		}
	}

	if( !InScale.IsZero() )
	{
		float ScaleU = static_cast<float>( InScale.X / GEditor->GetGridSize() );
		float ScaleV = static_cast<float>( InScale.Y / GEditor->GetGridSize() );

		ScaleU = 1.f - (ScaleU / 100.f);
		ScaleV = 1.f - (ScaleV / 100.f);

		// Ensure each polygon has unique texture vector indices.
		for( FConstLevelIterator Iterator = InViewportClient->GetWorld()->GetLevelIterator(); Iterator; ++Iterator )
		{
			UModel* Model = (*Iterator)->Model;
			for(int32 SurfaceIndex = 0;SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
			{
				FBspSurf& Surf = Model->Surfs[SurfaceIndex];
				if(Surf.PolyFlags & PF_Selected)
				{
					const FVector3f TextureU = Model->Vectors[Surf.vTextureU];
					const FVector3f TextureV = Model->Vectors[Surf.vTextureV];

					Surf.vTextureU = Model->Vectors.Add(TextureU);
					Surf.vTextureV = Model->Vectors.Add(TextureV);
				}
			}
			GEditor->polyTexScale( Model, ScaleU, 0.f, 0.f, ScaleV, false );
		}

	}

	return true;
}

