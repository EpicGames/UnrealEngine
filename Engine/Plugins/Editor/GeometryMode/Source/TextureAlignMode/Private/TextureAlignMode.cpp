// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureAlignMode.h"
#include "EditorModeManager.h"
#include "TextureAlignEdMode.h"
#include "GeometryModeModule.h"

#define LOCTEXT_NAMESPACE "TextureAlignMode"


FName FTextureAlignMode::GetToolkitFName() const
{
	return FName("TextureAlignMode");
}

FText FTextureAlignMode::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Texture Align Mode");
}

class FEdMode* FTextureAlignMode::GetEditorMode() const
{
	return (FEdModeTexture*)GLevelEditorModeTools().GetActiveMode(FGeometryEditingModes::EM_TextureAlign);
}

#undef LOCTEXT_NAMESPACE
