// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "EdMode.h"
#include "EditorModeTools.h"

class FEditorViewportClient;
class FEdModeTexture;
class FScopedTransaction;
class FViewport;

/**
 * Texture mode module
 */
class FTextureAlignModeModule : public IModuleInterface
{
private:
	TSharedPtr<class FEdModeTexture> EdModeTexture;
public:
	/**
	 * Called right after the module's DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;};

/**
 * Allows texture alignment on BSP surfaces via the widget.
 */
class FEdModeTexture : public FEdMode
{
public:
	FEdModeTexture();
	virtual ~FEdModeTexture();

	/** Stores the coordinate system that was active when the mode was entered so it can restore it later. */
	ECoordSystem SaveCoordSystem;

	virtual void Enter() override;
	virtual void Exit() override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool GetCustomDrawingCoordinateSystem( FMatrix& InMatrix, void* InData ) override;
	virtual bool GetCustomInputCoordinateSystem( FMatrix& InMatrix, void* InData ) override;
	virtual EAxisList::Type GetWidgetAxisToDraw( UE::Widget::EWidgetMode InWidgetMode ) const override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool AllowWidgetMove() override { return false; }
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;

protected:
	/** The current transaction. */
	FScopedTransaction*		ScopedTransaction;
	/* The world that brush that we started tracking with belongs to. Cleared when tracking ends. */
	UWorld*		TrackingWorld;
};

/**
 * FModeTool_Texture
 */
class FModeTool_Texture : public FModeTool
{
public:
	FModeTool_Texture();

	virtual bool InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale);

	// override these to allow this tool to keep track of the users dragging during a single drag event
	virtual bool StartModify()	{ PreviousInputDrag = FVector::ZeroVector; return true; }
	virtual bool EndModify()	{ return true; }

private:
	FVector PreviousInputDrag;
};
