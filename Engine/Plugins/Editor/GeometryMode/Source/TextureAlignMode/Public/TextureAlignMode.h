// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"

/**
 * Mode Toolkit for the Texture Align Tools
 */
class FTextureAlignMode : public FModeToolkit
{
public:
	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual class FEdMode* GetEditorMode() const override;

private:

};
