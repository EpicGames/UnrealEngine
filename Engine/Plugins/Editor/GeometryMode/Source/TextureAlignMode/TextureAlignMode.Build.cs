// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureAlignMode : ModuleRules
{
	public TextureAlignMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"GeometryMode",
				"SlateCore",
				"UnrealEd",
				"GeometryMode",
			}
		);
	}
}
